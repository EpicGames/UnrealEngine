[CoreRedirects]
+ClassRedirects=(OldName="/Script/PoseSearchEditor.AnimationBlendStackGraph",NewName="/Script/BlendStackEditor.AnimationBlendStackGraph")
+ClassRedirects=(OldName="/Script/PoseSearchEditor.AnimationBlendStackGraphSchema",NewName="/Script/BlendStackEditor.AnimationBlendStackGraphSchema")
+ClassRedirects=(OldName="/Script/PoseSearchEditor.AnimGraphNode_BlendStack",NewName="/Script/BlendStackEditor.AnimGraphNode_BlendStack")
+ClassRedirects=(OldName="/Script/PoseSearchEditor.AnimGraphNode_BlendStackInput",NewName="/Script/BlendStackEditor.AnimGraphNode_BlendStackInput")
+ClassRedirects=(OldName="/Script/PoseSearchEditor.AnimGraphNode_BlendStackResult",NewName="/Script/BlendStackEditor.AnimGraphNode_BlendStackResult")
+ClassRedirects=(OldName="/Script/PoseSearch.BlendStackAnimNodeLibrary",NewName="/Script/BlendStack.BlendStackAnimNodeLibrary")
+StructRedirects=(OldName="/Script/PoseSearch.AnimNode_BlendStack",NewName="/Script/BlendStack.AnimNode_BlendStack")
+StructRedirects=(OldName="/Script/PoseSearch.AnimNode_BlendStackInput",NewName="/Script/BlendStack.AnimNode_BlendStackInput")
+StructRedirects=(OldName="/Script/PoseSearch.AnimNode_BlendStackResult",NewName="/Script/BlendStack.AnimNode_BlendStackResult")
