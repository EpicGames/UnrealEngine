// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class BlendStackEditor : ModuleRules
{
	public BlendStackEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimGraph",
				"AnimGraphRuntime",
				"AnimationCore",
				"Core",
				"CoreUObject",
				"Engine",
				"BlendStack",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BlueprintGraph",
				"SlateCore",
				"UnrealEd"
			}
		);
	}
}
