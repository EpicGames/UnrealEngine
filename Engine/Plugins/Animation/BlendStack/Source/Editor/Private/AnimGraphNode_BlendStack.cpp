// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendStack.h"
#include "AnimationBlendStackGraph.h"
#include "AnimationBlendStackGraphSchema.h"
#include "AnimGraphNode_BlendStackInput.h"
#include "AnimGraphNode_Root.h"
#include "EdGraphUtilities.h"
#include "IAnimBlueprintCompilationContext.h"
#include "K2Node_AnimNodeReference.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendStack)

#define LOCTEXT_NAMESPACE "AnimGraphNode_BlendStack"

void UAnimGraphNode_BlendStack::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

FLinearColor UAnimGraphNode_BlendStack::GetNodeTitleColor() const
{
	return FColor(86, 182, 194);
}

FText UAnimGraphNode_BlendStack::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Blend Stack");
}

FText UAnimGraphNode_BlendStack::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Blend Stack");
}

FText UAnimGraphNode_BlendStack::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Blend Stack");
}

void UAnimGraphNode_BlendStack_Base::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if (UE::Anim::IAnimRootMotionProvider::Get())
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

void UAnimGraphNode_BlendStack_Base::BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog)
{
	const FAnimNode_BlendStack_Standalone* BlendStackNode = GetBlendStackNode();
	check(BlendStackNode);
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	AnimBlueprint->FindOrAddGroup(BlendStackNode->GetGroupName());
}

void UAnimGraphNode_BlendStack_Base::ExpandGraphAndProcessNodes(
	int GraphIndex,
	UEdGraph* SourceGraph, 
	UAnimGraphNode_Base* SourceRootNode, TArrayView<UAnimGraphNode_BlendStackInput*> InputNodes,
	IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData,
	UAnimGraphNode_Base*& OutRootNode, TArrayView<UAnimGraphNode_BlendStackInput*> OutInputNodes)
{
	// Note: This is mostly copied from UAnimGraphNode_BlendSpaceGraphBase::ExpandGraphAndProcessNodes
	
	// Clone the nodes from the source graph
	// Note that we outer this graph to the ConsolidatedEventGraph to allow ExpansionStep to 
	// correctly retrieve the context for any expanded function calls (custom make/break structs etc.)
	UEdGraph* ClonedGraph = FEdGraphUtilities::CloneGraph(SourceGraph, InCompilationContext.GetConsolidatedEventGraph(), &InCompilationContext.GetMessageLog(), true);

	// Grab all the animation nodes and find the corresponding 
	// root node and input pose nodes in the cloned set
	TArray<UAnimGraphNode_Base*> AnimNodeList;

	const UObject* SourceRootObject = InCompilationContext.GetMessageLog().FindSourceObject(SourceRootNode);
	TArray<UObject*> SourceInputObjects;
	SourceInputObjects.SetNum(InputNodes.Num());
	for (int32 Index = 0; Index < InputNodes.Num(); ++Index)
	{
		SourceInputObjects[Index] = InCompilationContext.GetMessageLog().FindSourceObject(InputNodes[Index]);
	}
	
	TSet<FName> TagsToRemap;

	for (auto NodeIt = ClonedGraph->Nodes.CreateIterator(); NodeIt; ++NodeIt)
	{
		UEdGraphNode* ClonedNode = *NodeIt;
		if (UAnimGraphNode_Base* TestNode = Cast<UAnimGraphNode_Base>(ClonedNode))
		{
			AnimNodeList.Add(TestNode);
			
			FName NodeTag = TestNode->GetTag();
			if (!NodeTag.IsNone())
			{
				TagsToRemap.Add(NodeTag);
				NodeTag.SetNumber(GraphIndex + 1);
				TestNode->SetTag(NodeTag);
			}

			//@TODO: There ought to be a better way to determine this
			UObject* TestObject = InCompilationContext.GetMessageLog().FindSourceObject(TestNode);
			if (TestObject == SourceRootObject)
			{
				OutRootNode = TestNode;
			}

			int32 FoundIndex;
			if (SourceInputObjects.Find(TestObject, FoundIndex))
			{
				OutInputNodes[FoundIndex] = (UAnimGraphNode_BlendStackInput*)TestNode;
			}
		}
	}

	check(OutRootNode && !OutInputNodes.Contains(nullptr));

	for (auto NodeIt = ClonedGraph->Nodes.CreateIterator(); NodeIt; ++NodeIt)
	{
		UEdGraphNode* ClonedNode = *NodeIt;

		if (UK2Node_AnimNodeReference* ReferenceNode = Cast<UK2Node_AnimNodeReference>(ClonedNode))
		{
			FName NodeTag = ReferenceNode->GetTag();
			if (TagsToRemap.Contains(NodeTag))
			{
				NodeTag.SetNumber(GraphIndex + 1);
				ReferenceNode->SetTag(NodeTag);
			}
		}
	}

	// Move the cloned nodes into the consolidated event graph
	const bool bIsLoading = InCompilationContext.GetBlueprint()->bIsRegeneratingOnLoad || IsAsyncLoading();
	const bool bIsCompiling = InCompilationContext.GetBlueprint()->bBeingCompiled;
	ClonedGraph->MoveNodesToAnotherGraph(InCompilationContext.GetConsolidatedEventGraph(), bIsLoading, bIsCompiling);

	// Process any animation nodes
	{
		TArray<UAnimGraphNode_Base*> RootSet;
		RootSet.Add(OutRootNode);

		InCompilationContext.PruneIsolatedAnimationNodes(RootSet, AnimNodeList);
		InCompilationContext.ProcessAnimationNodes(AnimNodeList);
	}
	
	// Run another expansion pass to catch the graph we just added (this is slightly wasteful)
	InCompilationContext.ExpansionStep(ClonedGraph, false);
	
	// Validate graph now we have expanded/pruned
	InCompilationContext.ValidateGraphIsWellFormed(ClonedGraph);
}

void UAnimGraphNode_BlendStack_Base::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	Super::OnProcessDuringCompilation(InCompilationContext, OutCompiledData);

	FAnimNode_BlendStack_Standalone* AnimNode = GetBlendStackNode();
	check(AnimNode);

	const int32 MaxBlendsNum = GetMaxActiveBlends();
	// Set MaxActiveBlends of the blend stack at compile time.
	// @todo: Allow reducing the blend stack size (but not decreasing), to enable scalability control under i.e. lower LODs
	AnimNode->SetMaxActiveBlends(MaxBlendsNum);

	UAnimationBlendStackGraph* SampleGraph = CastChecked<UAnimationBlendStackGraph>(BoundGraph);
	if (!SampleGraph->ResultNode->IsNodeRootSet())
	{
		// Input Pose is connected to Output Pose, so the sample graph does nothing.
		// No need to use allocate sample graphs in that case.
		AnimNode->PerSampleGraphPoseLinks.Reset();
		return;
	}

	// Allocate one sample graph per-active blend plus an extra one for the stored pose.
	const int32 NumSampleGraphs = MaxBlendsNum + 1;
	AnimNode->PerSampleGraphPoseLinks.SetNum(NumSampleGraphs);

	TArray<UAnimGraphNode_BlendStackInput*> InputNodes;
	BoundGraph->GetNodesOfClass<UAnimGraphNode_BlendStackInput>(InputNodes);

	int32 BlendStackAllocationIndex = InCompilationContext.GetAllocationIndexOfNode(this);

	FStructProperty* NodeProperty = GetFNodeProperty();
	check(NodeProperty);
	FArrayProperty* PoseLinksProperty = CastFieldChecked<FArrayProperty>(NodeProperty->Struct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimNode_BlendStack_Standalone, PerSampleGraphPoseLinks)));
	check(PoseLinksProperty);

	for(int32 Index = 0; Index < NumSampleGraphs; ++Index)
	{
		UAnimGraphNode_Base* ClonedRootNode;
		TArray<UAnimGraphNode_BlendStackInput*> ClonedInputNodes;
		ClonedInputNodes.SetNum(InputNodes.Num());

		ExpandGraphAndProcessNodes(Index, SampleGraph, SampleGraph->ResultNode, InputNodes, InCompilationContext, OutCompiledData, ClonedRootNode, ClonedInputNodes);

		// Blend stack node is potentially nested in the struct, so we can't use FPoseLinkMappingRecord. Patch at runtime instead.
		for (UAnimGraphNode_BlendStackInput* InputNode : ClonedInputNodes)
		{
			InputNode->Node.BlendStackAllocationIndex = BlendStackAllocationIndex;
			InputNode->Node.SampleIndex = Index;
		}

		InCompilationContext.AddPoseLinkMappingRecord(FPoseLinkMappingRecord::MakeFromArrayEntry(this, ClonedRootNode, PoseLinksProperty, Index));
	}
}

bool UAnimGraphNode_BlendStack_Base::DoesSupportTimeForTransitionGetter() const
{
	return true;
}

UAnimationAsset* UAnimGraphNode_BlendStack_Base::GetAnimationAsset() const
{
	return nullptr;
}

const TCHAR* UAnimGraphNode_BlendStack_Base::GetTimePropertyName() const
{
	return TEXT("InternalTimeAccumulator");
}

UScriptStruct* UAnimGraphNode_BlendStack_Base::GetTimePropertyStruct() const
{
	return FAnimNode_BlendStack::StaticStruct();
}

void UAnimGraphNode_BlendStack_Base::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading() && (BoundGraph == nullptr))
	{
		// Update older nodes without a graph to create one on load.
		CreateGraph();
	}
}

int32 UAnimGraphNode_BlendStack_Base::GetMaxActiveBlends() const
{
	FAnimNode_BlendStack_Standalone* AnimNode = GetBlendStackNode();
	check(AnimNode);
	return AnimNode->GetMaxActiveBlends();
}

void UAnimGraphNode_BlendStack_Base::CreateGraph()
{
	// Create a new animation graph
	check(BoundGraph == nullptr);
	BoundGraph = FBlueprintEditorUtils::CreateNewGraph(
		this,
		NAME_None,
		UAnimationBlendStackGraph::StaticClass(),
		UAnimationBlendStackGraphSchema::StaticClass());
	check(BoundGraph);

	// Initialize the anim graph
	const UEdGraphSchema* Schema = BoundGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*BoundGraph);

	// Add the new graph as a child of our parent graph
	UEdGraph* ParentGraph = GetGraph();
	if(ParentGraph->SubGraphs.Find(BoundGraph) == INDEX_NONE)
	{
		ParentGraph->Modify();
		ParentGraph->SubGraphs.Add(BoundGraph);
	}
}

void UAnimGraphNode_BlendStack_Base::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	CreateGraph();
}

TArray<UEdGraph*> UAnimGraphNode_BlendStack_Base::GetSubGraphs() const
{
	return TArray<UEdGraph*>({BoundGraph});
}

UObject* UAnimGraphNode_BlendStack_Base::GetJumpTargetForDoubleClick() const
{
	// Open the blend stack graph
	return BoundGraph;
}

void UAnimGraphNode_BlendStack_Base::JumpToDefinition() const
{
	if (UObject* HyperlinkTarget = GetJumpTargetForDoubleClick())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(HyperlinkTarget);
	}
}

void UAnimGraphNode_BlendStack_Base::DestroyNode()
{
	UBlueprint* Blueprint = GetBlueprint();
	UEdGraph* GraphToRemove = BoundGraph;
	BoundGraph = nullptr;
	Super::DestroyNode();

	if (GraphToRemove)
	{
		GraphToRemove->Modify();
		FBlueprintEditorUtils::RemoveGraph(Blueprint, GraphToRemove, EGraphRemoveFlags::Recompile);
	}
}

void UAnimGraphNode_BlendStack_Base::PostPasteNode()
{
	Super::PostPasteNode();

	if(BoundGraph)
	{
		// Add the new graph as a child of our parent graph
		UEdGraph* ParentGraph = GetGraph();

		if(ParentGraph->SubGraphs.Find(BoundGraph) == INDEX_NONE)
		{
			ParentGraph->SubGraphs.Add(BoundGraph);
		}

		for (UEdGraphNode* GraphNode : BoundGraph->Nodes)
		{
			GraphNode->CreateNewGuid();
			GraphNode->PostPasteNode();
			GraphNode->ReconstructNode();
		}

		//restore transactional flag that is lost during copy/paste process
		BoundGraph->SetFlags(RF_Transactional);
	}
}

#undef LOCTEXT_NAMESPACE
