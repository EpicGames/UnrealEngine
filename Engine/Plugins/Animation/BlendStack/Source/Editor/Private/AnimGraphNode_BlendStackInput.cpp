// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendStackInput.h"
#include "AnimGraphNode_BlendStack.h"
#include "AnimationBlendStackGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendStackInput)

#define LOCTEXT_NAMESPACE "BlendStackInput"

FText UAnimGraphNode_BlendStackInput::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText DefaultTitle = LOCTEXT("Title", "Blend Stack Input");
	return DefaultTitle;
}

FText UAnimGraphNode_BlendStackInput::GetMenuCategory() const
{
	return LOCTEXT("BlendStackInputCategory", "Animation|Blend Stack");
}

bool UAnimGraphNode_BlendStackInput::CanUserDeleteNode() const
{
	return true;
}

bool UAnimGraphNode_BlendStackInput::CanDuplicateNode() const
{
	return true;
}

bool UAnimGraphNode_BlendStackInput::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	return Graph->IsA(UAnimationBlendStackGraph::StaticClass());
}

#undef LOCTEXT_NAMESPACE
