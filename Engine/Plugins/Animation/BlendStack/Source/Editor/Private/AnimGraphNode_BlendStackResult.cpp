// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendStackResult.h"
#include "AnimGraphNode_BlendStackInput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendStackResult)

bool UAnimGraphNode_BlendStackResult::IsNodeRootSet() const
{
	if (!Pins.IsEmpty() && !Pins[0]->LinkedTo.IsEmpty())
	{
		if (UAnimGraphNode_BlendStackInput* InputNode = Cast<UAnimGraphNode_BlendStackInput>(Pins[0]->LinkedTo[0]->GetOuter()))
		{
			// If output is connected to input directly, graph is a no-op, unless OverridePlayrate is true. Consider it not set for pruning purposes.
			return InputNode->Node.bOverridePlayRate;
		}
	}

	return true;
}
