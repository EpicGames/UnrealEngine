// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBlendStackGraphSchema.h"
#include "AnimationBlendStackGraph.h"
#include "AnimGraphNode_BlendStackInput.h"
#include "AnimGraphNode_BlendStackResult.h"

/////////////////////////////////////////////////////
// UAnimationBlendStackGraphSchema

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationBlendStackGraphSchema)

void UAnimationBlendStackGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	UAnimationBlendStackGraph* TypedGraph = CastChecked<UAnimationBlendStackGraph>(&Graph);

	UAnimGraphNode_BlendStackResult* ResultNode;
	UAnimGraphNode_BlendStackInput* InputNode;

	{
		// Create the result node
		FGraphNodeCreator<UAnimGraphNode_BlendStackResult> NodeCreator(Graph);
		ResultNode = NodeCreator.CreateNode();
		NodeCreator.Finalize();
		SetNodeMetaData(ResultNode, FNodeMetadata::DefaultGraphNode);
		TypedGraph->ResultNode = ResultNode;
	}

	{
		// Create the result node
		FGraphNodeCreator<UAnimGraphNode_BlendStackInput> NodeCreator(Graph);
		InputNode = NodeCreator.CreateNode();
		NodeCreator.Finalize();
		SetNodeMetaData(InputNode, FNodeMetadata::DefaultGraphNode);

		// Move input node to the left of the output pose.
		InputNode->NodePosX -= 200;
	}

	// Connect input node to output pose.
	InputNode->Pins[0]->MakeLinkTo(ResultNode->Pins[0]);
}
