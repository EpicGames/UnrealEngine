// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendStackEditor.h"
#include "Modules/ModuleManager.h"


DEFINE_LOG_CATEGORY(LogBlendStackEditor);

#define LOCTEXT_NAMESPACE "BlendStackEditorModule"

namespace UE::BlendStack
{

//////////////////////////////////////////////////////////////////////////
// Blend Stack Editor Module
class FEditorModule : public IBlendStackEditorModuleInterface
{
public:
	void StartupModule() override;
	void ShutdownModule() override;
};

void FEditorModule::StartupModule()
{
}

void FEditorModule::ShutdownModule()
{

}


} // namespace UE::BlendStack

IMPLEMENT_MODULE(UE::BlendStack::FEditorModule, BlendStackEditor);

#undef LOCTEXT_NAMESPACE
