// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_AssetPlayerBase.h"
#include "BlendStack/AnimNode_BlendStack.h"
#include "AnimGraphNode_BlendStack.generated.h"

#define UE_API BLENDSTACKEDITOR_API

class UAnimGraphNode_BlendStackInput;

UCLASS(MinimalAPI, Abstract)
class UAnimGraphNode_BlendStack_Base : public UAnimGraphNode_AssetPlayerBase
{
	GENERATED_BODY()

public:
	UE_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	UE_API virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;

	UE_API virtual bool DoesSupportTimeForTransitionGetter() const override;
	UE_API virtual UAnimationAsset* GetAnimationAsset() const override;
	UE_API virtual const TCHAR* GetTimePropertyName() const override;
	UE_API virtual UScriptStruct* GetTimePropertyStruct() const override;

	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual void JumpToDefinition() const override;
	UE_API virtual void DestroyNode() override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual TArray<UEdGraph*> GetSubGraphs() const override;
	UE_API virtual void BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog) override;

protected:
	// Helper function for compilation
	UE_API void ExpandGraphAndProcessNodes(
		int GraphIndex,
		UEdGraph* SourceGraph, 
		UAnimGraphNode_Base* SourceRootNode, TArrayView<UAnimGraphNode_BlendStackInput*> SourceInputNode,
		IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData,
		UAnimGraphNode_Base*& OutRootNode, TArrayView<UAnimGraphNode_BlendStackInput*> OutInputNodes);

	virtual FAnimNode_BlendStack_Standalone* GetBlendStackNode() const PURE_VIRTUAL(UAnimGraphNode_BlendStack_Base::GetBlendStackNode, return nullptr;);
	UE_API int32 GetMaxActiveBlends() const;

private:

	UE_API void CreateGraph();

	UPROPERTY()
	TObjectPtr<UEdGraph> BoundGraph = nullptr;
};

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendStack : public UAnimGraphNode_BlendStack_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_BlendStack Node;

	virtual void Serialize(FArchive& Ar) override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;

protected:
	virtual FAnimNode_BlendStack_Standalone* GetBlendStackNode() const override { return (FAnimNode_BlendStack_Standalone*)(&Node); }
};

#undef UE_API
