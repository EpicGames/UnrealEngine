// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "BlendStack/AnimNode_BlendStackInput.h"

#include "AnimGraphNode_BlendStackInput.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendStackInput : public UAnimGraphNode_Base
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Inputs")
	FAnimNode_BlendStackInput Node;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;

	virtual bool CanUserDeleteNode() const override;
	virtual bool CanDuplicateNode() const override;
	virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const override;

	/** @return whether this node is editable. Non-editable nodes are implemented from function interfaces */
	bool IsEditable() const { return CanUserDeleteNode(); }

};