// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Root.h"

#include "AnimGraphNode_BlendStackResult.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendStackResult : public UAnimGraphNode_Root
{
	GENERATED_BODY()

	virtual bool IsNodeRootSet() const override;
};