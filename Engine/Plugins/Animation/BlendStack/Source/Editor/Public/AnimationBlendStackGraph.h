// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationGraph.h"
#include "AnimationBlendStackGraph.generated.h"

class UObject;

/** Animation graph used for blend stacks.
*	The result node is the root of the blend graph.
*	The input node is dynamically linked to the blend stack's sample pose.
*/
UCLASS(MinimalAPI)
class UAnimationBlendStackGraph : public UAnimationGraph
{
	GENERATED_BODY()

public:
	// Result node within the state's animation graph
	UPROPERTY()
	TObjectPtr<class UAnimGraphNode_Root> ResultNode = nullptr;
};

