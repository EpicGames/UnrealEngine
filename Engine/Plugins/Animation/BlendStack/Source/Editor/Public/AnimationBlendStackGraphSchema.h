// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationGraphSchema.h"
#include "AnimationBlendStackGraphSchema.generated.h"

UCLASS(MinimalAPI)
class UAnimationBlendStackGraphSchema : public UAnimationGraphSchema
{
	GENERATED_BODY()

	//~ Begin UEdGraphSchema Interface.
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	//~ End UEdGraphSchema Interface.
};
