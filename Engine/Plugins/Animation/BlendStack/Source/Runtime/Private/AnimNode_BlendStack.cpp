// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendStack/AnimNode_BlendStack.h"
#include "Algo/MaxElement.h"
#include "Animation/AnimBlendDebugScope.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimInertializationSyncScope.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimPoseSearchProvider.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimTrace.h"
#include "Animation/BlendSpace.h"
#include "Animation/NamedValueArray.h"
#include "BlendedCurveUtils.h"
#include "BlendStack/BlendStackDefines.h"
#include "BlendStackAnimEventsFilterScope.h"
#include "ObjectTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendStack)

#define LOCTEXT_NAMESPACE "AnimNode_BlendStack"

namespace UE::BlendStack
{
#if ENABLE_ANIM_DEBUG
	static bool GVarAnimBlendStackEnable = true;
	static FAutoConsoleVariableRef CVarAnimBlendStackEnable(TEXT("a.AnimNode.BlendStack.Enable"), GVarAnimBlendStackEnable, TEXT("Enable / Disable Blend Stack"));
#endif
} // namespace UE::BlendStack

/////////////////////////////////////////////////////
// FBlendStackAnimPlayer
void FBlendStackAnimPlayer::Initialize(const FAnimationInitializeContext& Context, UAnimationAsset* AnimationAsset, float AccumulatedTime, bool bLoop,
	bool bMirrored, UMirrorDataTable* MirrorDataTable, float BlendTime, const UBlendProfile* InBlendProfile, EAlphaBlendOption InBlendOption,
	const FVector& BlendParameters, float PlayRate, float ActivationDelay, int32 InPoseLinkIdx, FName GroupName, EAnimGroupRole::Type GroupRole, EAnimSyncMethod GroupMethod, bool bOverridePositionWhenJoiningSyncGroupAsLeader)
{
	if (bMirrored && !MirrorDataTable)
	{
		UE_LOGF(LogBlendStack, Error, "FBlendStackAnimPlayer::Initialize failed for '%ls'. Mirroring will not work because MirrorDataTable is missing", *GetNameSafe(AnimationAsset));
	}
	
	check(Context.AnimInstanceProxy);
	USkeleton* Skeleton = Context.AnimInstanceProxy->GetSkeleton();
	check(Skeleton);

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	const int32 NumSkeletonBones = RefSkeleton.GetNum();
	if (NumSkeletonBones <= 0)
	{
		UE_LOGF(LogBlendStack, Error, "FBlendStackAnimPlayer::Initialize failed for '%ls'. Skeleton has no bones?!", *GetNameSafe(AnimationAsset));
	}
	else if (BlendTime > UE_KINDA_SMALL_NUMBER)
	{
		// handling BlendTime > 0 and RootBoneBlendTime >= 0
		if (InBlendProfile != nullptr)
		{
			TotalBlendInTimePerBone.SetNumUninitialized(NumSkeletonBones);
			InBlendProfile->FillSkeletonBoneDurationsArray(TotalBlendInTimePerBone, BlendTime, Skeleton);
		}
	}

	BlendOption = InBlendOption;
	BlendProfile = InBlendProfile;

	TotalBlendInTime = BlendTime;
	CurrentBlendInTime = 0.f;
	TimeToActivation = ActivationDelay;

	MirrorNode.SetMirrorDataTable(MirrorDataTable);
	MirrorNode.SetMirror(bMirrored);
	
	bool bUnsupportedAnimAsset = false;
	if (Cast<UAnimMontage>(AnimationAsset))
	{
		bUnsupportedAnimAsset = true;
	}
	else if (UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimationAsset))
	{
		BlendSpacePlayerNode.SetBlendSpace(nullptr);

		SequencePlayerNode.SetAccumulatedTime(AccumulatedTime);
		SequencePlayerNode.SetSequence(SequenceBase);
		SequencePlayerNode.SetLoopAnimation(bLoop);
		SequencePlayerNode.SetPlayRate(PlayRate);
		SequencePlayerNode.SetGroupMethod(GroupMethod);
		SequencePlayerNode.SetGroupName(GroupName);
		SequencePlayerNode.SetGroupRole(GroupRole);
		SequencePlayerNode.SetOverridePositionWhenJoiningSyncGroupAsLeader(bOverridePositionWhenJoiningSyncGroupAsLeader);
	}
	else if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAsset))
	{
		SequencePlayerNode.SetSequence(nullptr);

		// making sure AccumulatedTime is in normalized space
		AccumulatedTime = FMath::Clamp(AccumulatedTime, 0.f, 1.f);

		BlendSpacePlayerNode.SetResetPlayTimeWhenBlendSpaceChanges(false /*!bReset*/);
		BlendSpacePlayerNode.SetAccumulatedTime(AccumulatedTime);
		BlendSpacePlayerNode.SetBlendSpace(BlendSpace);
		BlendSpacePlayerNode.SetLoop(bLoop);
		BlendSpacePlayerNode.SetPlayRate(PlayRate);
		BlendSpacePlayerNode.SetPosition(BlendParameters);
		BlendSpacePlayerNode.SetGroupMethod(GroupMethod);
		BlendSpacePlayerNode.SetGroupName(GroupName);
		BlendSpacePlayerNode.SetGroupRole(GroupRole);
		BlendSpacePlayerNode.SetOverridePositionWhenJoiningSyncGroupAsLeader(bOverridePositionWhenJoiningSyncGroupAsLeader);
	}
	else if (AnimationAsset)
	{
		bUnsupportedAnimAsset = true;
	}

	if (bUnsupportedAnimAsset)
	{
		BlendSpacePlayerNode.SetBlendSpace(nullptr);
		SequencePlayerNode.SetSequence(nullptr);

		UE_LOGF(LogBlendStack, Error, "FBlendStackAnimPlayer::Initialize unsupported AnimationAsset %ls", *GetNameSafe(AnimationAsset));
	}

	UpdateSourceLinkNode();
	PoseLinkIndex = InPoseLinkIdx;

	OverrideCurve.Empty();

#if OBJECT_TRACE_ENABLED && (ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG)
	// Matches 'MakeBlendWeightCurveColor' in DebugWeightsTrack.cpp to match rewind debugger color (Does not depend since do not want dependency on debug for color).
	auto MakeDebugColor = [](uint32 InSeed)
	{
		FRandomStream Stream(InSeed);
		const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
		const uint8 SatVal = 196;
		return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
	};

	uint64 AssetId = FObjectTrace::GetObjectId(AnimationAsset);
	DebugColor = MakeDebugColor(CityHash32(reinterpret_cast<char*>(&AssetId), 8)).ToFColor(true);
#endif
}

float FBlendStackAnimPlayer::GetSecondsFromEnd() const
{
	float TimeFromEndInSeconds = 0.f;

	if (SequencePlayerNode.GetSequence())
	{
		const float AccumulatedTime = SequencePlayerNode.GetAccumulatedTime();
		TimeFromEndInSeconds = SequencePlayerNode.GetTimeFromEnd(AccumulatedTime);
	}
	else if (const UBlendSpace* BlendSpace = BlendSpacePlayerNode.GetBlendSpace())
	{
		int32 TriangulationIndex = 0;
		TArray<FBlendSampleData> BlendSamples;
		const FVector BlendParameters = BlendSpacePlayerNode.GetPosition();
		BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamples, TriangulationIndex, true);
		const float PlayLength = BlendSpace->GetAnimationLengthFromSampleData(BlendSamples);
		const float AccumulatedTime = BlendSpacePlayerNode.GetAccumulatedTime();
		TimeFromEndInSeconds = BlendSpacePlayerNode.GetTimeFromEnd(AccumulatedTime) * PlayLength;
	}

	return TimeFromEndInSeconds;
}

void FBlendStackAnimPlayer::UpdatePlayRate(float PlayRate)
{
	if (SequencePlayerNode.GetSequence())
	{
		SequencePlayerNode.SetPlayRate(PlayRate);
	}
	else if (BlendSpacePlayerNode.GetBlendSpace())
	{
		BlendSpacePlayerNode.SetPlayRate(PlayRate);
	}
}

void FBlendStackAnimPlayer::StorePoseContext(const FPoseContext& PoseContext)
{
	SequencePlayerNode.SetSequence(nullptr);
	BlendSpacePlayerNode.SetBlendSpace(nullptr);
	MirrorNode.SetSourceLinkNode(nullptr);

	if (PoseContext.Pose.IsValid())
	{
		StoredBones = PoseContext.Pose.GetBones();
		StoredBoneContainer = PoseContext.Pose.GetBoneContainer();
	}

	StoredCurve.CopyFrom(PoseContext.Curve);
	StoredAttributes.CopyFrom(PoseContext.CustomAttributes, PoseContext.Pose.GetBoneContainer());
}

bool FBlendStackAnimPlayer::HasValidPoseContext() const
{
	return !StoredBones.IsEmpty() && StoredBoneContainer.IsValid();
}

void FBlendStackAnimPlayer::MovePoseContextTo(FBlendStackAnimPlayer& Other)
{
	// moving the allocated memory to Other
	Other.StoredBones = MoveTemp(StoredBones);
	Other.StoredCurve = MoveTemp(StoredCurve);
	Other.StoredAttributes = MoveTemp(StoredAttributes);
	Other.StoredBoneContainer = MoveTemp(StoredBoneContainer);
	
	// making sure Other pose context is invalid
	Other.StoredBones.Reset();
}

void FBlendStackAnimPlayer::RestorePoseContext(FPoseContext& PoseContext) const
{
	check(!SequencePlayerNode.GetSequence() && !BlendSpacePlayerNode.GetBlendSpace());

	if (StoredBoneContainer.IsValid())
	{
		// Serial number mismatch means a potential bone LOD mismatch, even if we have the same number of bones.
		// Remap the pose manually in those cases.
		if (PoseContext.Pose.GetBoneContainer().GetSerialNumber() == StoredBoneContainer.GetSerialNumber())
		{
			if (StoredBones.IsEmpty())
			{
				PoseContext.ResetToRefPose();
			}
			else
			{
				check(PoseContext.Pose.GetNumBones() == StoredBones.Num());
				FMemory::Memcpy(PoseContext.Pose.GetMutableBones().GetData(), StoredBones.GetData(), sizeof(FTransform) * PoseContext.Pose.GetNumBones());
			}
		}
		else
		{
			const FBoneContainer CurrentBoneContainer = PoseContext.Pose.GetBoneContainer();
			for (FCompactPoseBoneIndex CompactPoseIndex : PoseContext.Pose.ForEachBoneIndex())
			{
				// Map the current compact pose index to skeleton index, and map this back to the stored compact pose index.
				const FSkeletonPoseBoneIndex SkeletonPoseIndex =  CurrentBoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(CompactPoseIndex);
				const FCompactPoseBoneIndex StoredCompactPoseIndex =  StoredBoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonPoseIndex);
				if (StoredCompactPoseIndex == INDEX_NONE)
				{
					// If our stored pose doesn't have the bone, reset to ref pose.
					PoseContext.Pose[CompactPoseIndex] = CurrentBoneContainer.GetRefPoseTransform(CompactPoseIndex);
				}
				else
				{
					PoseContext.Pose[CompactPoseIndex] = StoredBones[StoredCompactPoseIndex.GetInt()];
				}
			}
		}
	}
	else
	{
		PoseContext.ResetToRefPose();
	}
	
	PoseContext.Curve.CopyFrom(StoredCurve);
	PoseContext.CustomAttributes.CopyFrom(StoredAttributes, PoseContext.Pose.GetBoneContainer());
}

// @todo: maybe implement copy/move constructors and assignment operator do so (or use a list instead of an array)
// since we're making copies and moving this object in memory, we're using this method to set the MirrorNode SourceLinkNode when necessary
void FBlendStackAnimPlayer::UpdateSourceLinkNode()
{
	if (SequencePlayerNode.GetSequence())
	{
		MirrorNode.SetSourceLinkNode(&SequencePlayerNode);
	}
	else if (BlendSpacePlayerNode.GetBlendSpace())
	{
		MirrorNode.SetSourceLinkNode(&BlendSpacePlayerNode);
	}
	else
	{
		MirrorNode.SetSourceLinkNode(nullptr);
	}
}

bool FBlendStackAnimPlayer::IsLooping() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.IsLooping();
	}

	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.IsLooping();
	}

	return false;
}

void FBlendStackAnimPlayer::Evaluate_AnyThread(FPoseContext& Output)
{
	if (SequencePlayerNode.GetSequence() || BlendSpacePlayerNode.GetBlendSpace())
	{
		UpdateSourceLinkNode();
		MirrorNode.Evaluate_AnyThread(Output);

		if (OverrideCurve.Num() != 0)
		{
			UE::Anim::FNamedValueArrayUtils::Union(Output.Curve, OverrideCurve);
		}
	}
	else
	{
		RestorePoseContext(Output);
	}
}

void FBlendStackAnimPlayer::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	UpdateSourceLinkNode();
	MirrorNode.Update_AnyThread(Context);
}

float FBlendStackAnimPlayer::GetAccumulatedTime() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetAccumulatedTime();
	}
	
	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		// making sure BlendSpacePlayerNode.GetAccumulatedTime() is in normalized space
		check(BlendSpacePlayerNode.GetAccumulatedTime() >= 0.f && BlendSpacePlayerNode.GetAccumulatedTime() <= 1.f);
		return BlendSpacePlayerNode.GetAccumulatedTime();
	}

	return 0.f;
}

EAnimSyncMethod FBlendStackAnimPlayer::GetGroupMethod() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetGroupMethod();
	}
	
	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetGroupMethod();
	}

	return EAnimSyncMethod::DoNotSync;
}

FName FBlendStackAnimPlayer::GetGroupName() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetGroupName();
	}
	
	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetGroupName();
	}

	return FName();
}

EAnimGroupRole::Type FBlendStackAnimPlayer::GetGroupRole() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetGroupRole();
	}
	
	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetGroupRole();
	}

	return EAnimGroupRole::Type::CanBeLeader;
}

bool FBlendStackAnimPlayer::GetOverridePositionWhenJoiningSyncGroupAsLeader() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetOverridePositionWhenJoiningSyncGroupAsLeader();
	}
	
	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetOverridePositionWhenJoiningSyncGroupAsLeader();
	}

	return false;
}

const UBlendProfile* FBlendStackAnimPlayer::GetBlendProfile() const
{
	return BlendProfile.Get();
}

UMirrorDataTable* FBlendStackAnimPlayer::GetMirrorDataTable() const
{
	return MirrorNode.GetMirrorDataTable();
}

float FBlendStackAnimPlayer::GetCurrentAssetTime() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetCurrentAssetTime();
	}

	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetCurrentAssetTime();
	}

	return 0.f;
}

float FBlendStackAnimPlayer::GetCurrentAssetLength() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetCurrentAssetLength();
	}

	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetCurrentAssetLength();
	}

	return 0.f;
}

float FBlendStackAnimPlayer::GetPlayRate() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetPlayRate();
	}

	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetPlayRate();
	}

	return 0.f;
}

FAnimNode_AssetPlayerBase* FBlendStackAnimPlayer::GetAssetPlayerNode()
{
	if (SequencePlayerNode.GetSequence())
	{
		return &SequencePlayerNode;
	}
	else if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return &BlendSpacePlayerNode;
	}

	// Anim player was initialized with an unsupported asset type.
	return nullptr;
}

#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG 
FColor FBlendStackAnimPlayer::GetDebugColor() const
{
	return DebugColor;
}
#endif

void FBlendStackAnimPlayer::UpdateWithDeltaTime(float DeltaTime, int32 PlayerDepth, float PlayerDepthBlendInTimeMultiplier)
{
	const bool bIsMainPlayer = PlayerDepth == 0;

	if (TimeToActivation > 0.f)
	{
		TimeToActivation -= DeltaTime;

		if (TimeToActivation < 0.f)
		{
			DeltaTime = -TimeToActivation;
			TimeToActivation = 0.f;
		}
		else
		{
			DeltaTime = 0.f;
		}
	}

	if (bIsMainPlayer)
	{
		CurrentBlendInTime += DeltaTime;
	}
	else
	{
		const float ScaledDeltaTime = DeltaTime * FMath::Pow(PlayerDepthBlendInTimeMultiplier, PlayerDepth + 1);
		CurrentBlendInTime += ScaledDeltaTime;
	}
}

FVector FBlendStackAnimPlayer::GetBlendParameters() const
{
	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetPosition();
	}

	return FVector::ZeroVector;
}

void FBlendStackAnimPlayer::SetBlendParameters(const FVector& BlendParameters)
{
	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		BlendSpacePlayerNode.SetPosition(BlendParameters);
	}
}

FString FBlendStackAnimPlayer::GetAnimationName() const
{
	if (SequencePlayerNode.GetSequence())
	{
		check(SequencePlayerNode.GetSequence());
		return SequencePlayerNode.GetSequence()->GetName();
	}

	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		check(BlendSpacePlayerNode.GetBlendSpace());
		return BlendSpacePlayerNode.GetBlendSpace()->GetName();
	}

	return FString("StoredPose");
}

UAnimationAsset* FBlendStackAnimPlayer::GetAnimationAsset() const
{
	if (SequencePlayerNode.GetSequence())
	{
		return SequencePlayerNode.GetSequence();
	}

	if (BlendSpacePlayerNode.GetBlendSpace())
	{
		return BlendSpacePlayerNode.GetBlendSpace();
	}

	return nullptr;
}

float FBlendStackAnimPlayer::GetBlendInPercentage() const
{
	if (TotalBlendInTime < UE_SMALL_NUMBER)
	{
		if (TimeToActivation > 0.f)
		{
			return 0.f;
		}
		return 1.f;
	}

	check(CurrentBlendInTime >= 0.f);
	return FMath::Min(CurrentBlendInTime / TotalBlendInTime, 1.f);
}

int32 FBlendStackAnimPlayer::GetBlendInWeightsNum() const
{
	return TotalBlendInTimePerBone.Num();
}

float FBlendStackAnimPlayer::GetBlendInWeight() const
{
	const float BlendInPercentage = GetBlendInPercentage();
	const float BlendInWeight = FAlphaBlend::AlphaToBlendOption(BlendInPercentage, GetBlendOption());
	return BlendInWeight;
}

void FBlendStackAnimPlayer::GetBlendInWeights(TArrayView<float> Weights) const
{
	check(Weights.Num() == GetBlendInWeightsNum());
	
	const float WeightForZeroBlendInTime = TimeToActivation > 0.f ? 0.f : 1.f;
	for (int32 BoneIdx = 0; BoneIdx < Weights.Num(); ++BoneIdx)
	{
		const float TotalBlendInTimeBoneIdx = TotalBlendInTimePerBone[BoneIdx];
		if (TotalBlendInTimeBoneIdx < UE_SMALL_NUMBER)
		{
			Weights[BoneIdx] = WeightForZeroBlendInTime;
		}
		else
		{
			check(CurrentBlendInTime >= 0.f);
			const float LinearWeight = FMath::Min(CurrentBlendInTime / TotalBlendInTimeBoneIdx, 1.f);
			Weights[BoneIdx] = FAlphaBlend::AlphaToBlendOption(LinearWeight, BlendOption);
		}
	}
}

// This method fills weights indexed by the compact skeleton indices
// Todo: move to struct method after 5.5. 
// This needs to be a free function for a hot fix to avoid breaking ABI
void GetBlendInWeightsCompactPose(const FBlendStackAnimPlayer& AnimPlayer, const TCustomBoneIndexArray<float, FSkeletonPoseBoneIndex>& TotalBlendInTimePerBone, TArrayView<float> Weights, const FBoneContainer& BoneContainer)
{	
	const int32 CompactPoseNumBones = BoneContainer.GetCompactPoseNumBones();
	check(Weights.Num() == CompactPoseNumBones);

	const float WeightForZeroBlendInTime = AnimPlayer.GetTimeToActivation() > 0.f ? 0.f : 1.f;
	for (int32 CompactPoseBoneIdx = 0; CompactPoseBoneIdx < CompactPoseNumBones; ++CompactPoseBoneIdx)
	{
		const FSkeletonPoseBoneIndex SkeletonBoneIdx = BoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(FCompactPoseBoneIndex(CompactPoseBoneIdx));
		if(!SkeletonBoneIdx.IsValid())
		{
			Weights[CompactPoseBoneIdx] = 0.0f;
			continue;
		}

		const float TotalBlendInTimeBoneIdx = TotalBlendInTimePerBone[SkeletonBoneIdx];
		if (TotalBlendInTimeBoneIdx < UE_SMALL_NUMBER)
		{
			Weights[CompactPoseBoneIdx] = WeightForZeroBlendInTime;
		}
		else
		{
			check(AnimPlayer.GetCurrentBlendInTime() >= 0.f);
			const float LinearWeight = FMath::Min(AnimPlayer.GetCurrentBlendInTime() / TotalBlendInTimeBoneIdx, 1.f);
			Weights[CompactPoseBoneIdx] = FAlphaBlend::AlphaToBlendOption(LinearWeight, AnimPlayer.GetBlendOption());
		}
	}
}

/////////////////////////////////////////////////////
// FAnimNode_BlendStack_Standalone


void FAnimNode_BlendStack_Standalone::UpdateBlendspaceParameters(const EBlendStack_BlendspaceUpdateMode UpdateMode, const FVector& BlendParameters)
{
	// Update blend space parameters
	if (UpdateMode == EBlendStack_BlendspaceUpdateMode::UpdateAll)
	{
		// apply blend space parameters to all blendspaces that are playing, including ones that are blending out
		for (FBlendStackAnimPlayer& Player : AnimPlayers)
		{
			Player.SetBlendParameters(BlendParameters);
		}
	}
	else if (UpdateMode == EBlendStack_BlendspaceUpdateMode::UpdateActiveOnly)
	{
		// apply blend space parameters only to the blendspace that is playing/blending in
		if (!AnimPlayers.IsEmpty())
		{
			AnimPlayers[0].SetBlendParameters(BlendParameters);
		}
	}
}

void FAnimNode_BlendStack_Standalone::PopLastAnimPlayer()
{
	const int32 LastAnimPlayerIndex = AnimPlayers.Num() - 1;

#if DO_CHECK
	for (int32 AnimPlayerIndex = 0; AnimPlayerIndex < LastAnimPlayerIndex; ++AnimPlayerIndex)
	{
		// making sure only the last AnimPlayer can have a valid pose
		check(!AnimPlayers[AnimPlayerIndex].HasValidPoseContext());
	}
#endif // DO_CHECK

	if (LastAnimPlayerIndex > 0 && AnimPlayers[LastAnimPlayerIndex].HasValidPoseContext())
	{
		AnimPlayers[LastAnimPlayerIndex].MovePoseContextTo(AnimPlayers[LastAnimPlayerIndex - 1]);
	}

	// popping the last anim player
	AnimPlayers.SetNum(LastAnimPlayerIndex);
}

void FAnimNode_BlendStack_Standalone::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendStack_Evaluate_AnyThread);

	Super::Evaluate_AnyThread(Output);

	bool bDisableBlendStack = false;
#if ENABLE_ANIM_DEBUG
	bDisableBlendStack = !UE::BlendStack::GVarAnimBlendStackEnable;

	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel))
	{
		// output current asset as "Asset" because that column is shown by default
		TRACE_ANIM_NODE_VALUE(Output, TEXT("Asset"), GetAnimAsset());

		for (int32 i = 0; i < AnimPlayers.Num(); ++i)
		{
			if (!AnimPlayers[i].IsPostponed())
			{
				FString IndexString = FString("[" + FString::FromInt(i) + FString("]"));
				FString Asset = FString("Asset") + IndexString;
				FString ElapsedTime = FString("ElapsedTime") + IndexString;
				FString CurrentBlendInTime = FString("CurrentBlendInTime") + IndexString;
				FString TotalBlendInTime = FString("TotalBlendInTime") + IndexString;
				FString TimeToActivation = FString("TimeToActivation") + IndexString;
				FString Mirror = FString("Mirror") + IndexString;

				const FBlendStackAnimPlayer& AnimPlayer = AnimPlayers[i];
				TRACE_ANIM_NODE_VALUE(Output, *Asset, AnimPlayer.GetAnimationAsset());
				TRACE_ANIM_NODE_VALUE(Output, *ElapsedTime, AnimPlayer.GetAccumulatedTime());
				TRACE_ANIM_NODE_VALUE(Output, *CurrentBlendInTime, AnimPlayer.GetCurrentBlendInTime());
				TRACE_ANIM_NODE_VALUE(Output, *TotalBlendInTime, AnimPlayer.GetTotalBlendInTime());
				TRACE_ANIM_NODE_VALUE(Output, *TimeToActivation, AnimPlayer.GetTimeToActivation());
			}
		}
	}
#endif // ENABLE_ANIM_DEBUG


	const int32 BlendStackSize = AnimPlayers.Num();
	if (BlendStackSize <= 0)
	{
		Output.ResetToRefPose();
	}
	else if (BlendStackSize == 1 || bDisableBlendStack)
	{
		if (!EvaluateSample(Output, 0))
		{
			Output.ResetToRefPose();
		}
	}
	else
	{
		// evaluating the last AnimPlayer into Output...
		if (!EvaluateSample(Output, BlendStackSize - 1))
		{
			UE_LOGF(LogBlendStack, Error, "FAnimNode_BlendStack_Standalone::Evaluate_AnyThread couldn't evaluate its last sample. Defaulting to RefPose");
			Output.ResetToRefPose();
		}

		FPoseContext EvaluationPoseContext(Output);

		const FBoneContainer& BoneContainer = Output.Pose.GetBoneContainer();
		const int32 NumCompactPoseBones = BoneContainer.GetCompactPoseNumBones();

		auto EvaluateAndBlendPlayerByIndex = [this, &EvaluationPoseContext, &Output, &BoneContainer, NumCompactPoseBones](int32 PlayerIndex)
		{
			FAnimationPoseData OutputAnimationPoseData(Output);
			FAnimationPoseData EvaluationAnimationPoseData(EvaluationPoseContext);

			// Evaluate into EvaluationPoseContext and then blend it with the Output (initialized with the last AnimPlayer evaluation)
			if (EvaluateSample(EvaluationPoseContext, PlayerIndex))
			{
				const int32 BlendInWeightsNum = AnimPlayers[PlayerIndex].GetBlendInWeightsNum();
				if (BlendInWeightsNum > 0)
				{
					TArrayView<float> EvaluationAnimationPoseDataWeights((float*)FMemory_Alloca(NumCompactPoseBones * sizeof(float)), NumCompactPoseBones);
					GetBlendInWeightsCompactPose(AnimPlayers[PlayerIndex], AnimPlayers[PlayerIndex].TotalBlendInTimePerBone, EvaluationAnimationPoseDataWeights, BoneContainer);
					BlendWithPosePerBone(OutputAnimationPoseData, EvaluationAnimationPoseData, EvaluationAnimationPoseDataWeights);
				}
				else
				{
					const float OutputAnimationPoseDataWeight = 1.f - AnimPlayers[PlayerIndex].GetBlendInWeight();
					BlendWithPose(OutputAnimationPoseData, EvaluationAnimationPoseData, OutputAnimationPoseDataWeight);
				}
			}
		};

		// ...continuing with the valuation and accumulation on the Output FPoseContext
		// of AnimPlayer(s) from the second last to the AnimPlayer[MaxActiveBlends].
		int32 PlayerIndex = BlendStackSize - 2;
		// Start evaluating with our least significant players.
		for (; PlayerIndex >= MaxActiveBlends; --PlayerIndex)
		{
			EvaluateAndBlendPlayerByIndex(PlayerIndex);

			// too many AnimPlayers! we don't have enough available blends to hold them all, so we accumulate the blended poses into Output / BlendedPoseContext.
			PopLastAnimPlayer();
		}

		// At this point Output FPoseContext contains all the weighted accumulated poses of the from AnimPlayer[MaxActiveBlends] to AnimPlayer[AnimPlayer.Num()-1]
		if (PlayerIndex == (MaxActiveBlends - 1))
		{
			check(AnimPlayers.Num() == MaxActiveBlends + 1);

			if (bStoreBlendedPose)
			{
				// We store Output / BlendedPoseContext into the last AnimPlayer, that will hold a static pose, no longer an animation playing.
				AnimPlayers.Last().StorePoseContext(Output);
			}
#if !NO_LOGGING
			// warning if we're dropping an animplayer with relevant (MaxBlendInTimeToOverrideAnimation) weight (GetBlendInPercentage)
			else if (AnimPlayers.Last().GetBlendInPercentage() < (1.f - MaxBlendInTimeToOverrideAnimation))
			{
				UE_LOGF(LogBlendStack, Warning, "FAnimNode_BlendStack_Standalone::Evaluate_AnyThread dropping animplayer with blend in at %.2f", AnimPlayers.Last().GetBlendInPercentage());
			}
#endif // !NO_LOGGING
		}

		// Continue with the evaluation of the most significant AnimPlayer(s) with the associated graphs
		for (; PlayerIndex >= 0; --PlayerIndex)
		{
			EvaluateAndBlendPlayerByIndex(PlayerIndex);
		}
	}
}

void FAnimNode_BlendStack_Standalone::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Super::Initialize_AnyThread(Context);

	if (bShouldFilterNotifies)
	{
		NotifiesFiredLastTick = MakeShared<TArray<FName>>();
		NotifyRecencyMap = MakeShared<TMap<FName, float>>();
	}
	
	Reset();

	if (PerSampleGraphPoseLinks.IsEmpty() == false)
	{
		SampleGraphExecutionHelpers.SetNum(PerSampleGraphPoseLinks.Num());
		for (UE::BlendStack::FBlendStack_SampleGraphExecutionHelper& ExecutionHelper : SampleGraphExecutionHelpers)
		{
			ExecutionHelper.CacheBoneCounter.Reset();
		}
	}
}

void FAnimNode_BlendStack_Standalone::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) 
{
	const int32 BlendStackSize = AnimPlayers.Num();
	for (int32 AnimPlayerIndex = 0; AnimPlayerIndex < BlendStackSize; ++AnimPlayerIndex)
	{
		// Cache bones for all active anim players.
		// There's no need to check for weight since all unneeded anim players 
		// would have been pruned during the last evaluation.
		CacheBonesForSample(Context, AnimPlayerIndex);
	}
}

void FAnimNode_BlendStack_Standalone::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendStack_UpdateAssetPlayer);

	Super::UpdateAssetPlayer(Context);

	if (bShouldFilterNotifies)
	{
		if (const UWorld* World = Context.GetAnimInstanceObject()->GetWorld())
		{
			const double CurrentGameTime = World->GetTimeSeconds();
		
			// Set target time-outs for notifies fired last tick.
			for (const FName & NotifyName : *NotifiesFiredLastTick)
			{
				NotifyRecencyMap->FindOrAdd(NotifyName) = CurrentGameTime + NotifyRecencyTimeOut;
			}
			NotifiesFiredLastTick->Reset();
			
			// Find notifies that have timed-out and should be allowed to fire this tick.
			for (auto It = NotifyRecencyMap->CreateIterator(); It; ++It)
			{
				if (CurrentGameTime >= It->Value)
				{
					It.RemoveCurrent();
				}
			}
		}
	}
	
	UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FBlendStackAnimEventsFilterScope> BlendStackInfoScope(bShouldFilterNotifies, Context, NotifiesFiredLastTick, NotifyRecencyMap);
		
	// AnimPlayers[0] is the most newly inserted AnimPlayer, AnimPlayers[AnimPlayers.Num()-1] is the oldest, so to calculate the weights
	// we ask AnimPlayers[0] its BlendInPercentage and then distribute the left over (CurrentWeightMultiplier) to the rest of the AnimPlayers
	// AnimPlayers[AnimPlayerIndex].GetBlendWeight() will now store the weighted contribution of AnimPlayers[AnimPlayerIndex] to be able to calculate root motion from animation
	float CurrentWeightMultiplier = 1.f;
	const int32 BlendStackSize = AnimPlayers.Num();
	int32 AnimPlayerIndex = 0;
	bool bLocalIsContextActive = true;
	while (AnimPlayerIndex < BlendStackSize)
	{
		FBlendStackAnimPlayer& AnimPlayer = AnimPlayers[AnimPlayerIndex];
		const bool bIsLastAnimPlayers = AnimPlayerIndex == BlendStackSize - 1;
		const float BlendInPercentage = bIsLastAnimPlayers ? 1.f : AnimPlayer.GetBlendInWeight();
		const float AnimPlayerBlendWeight = CurrentWeightMultiplier * BlendInPercentage;

		FAnimationUpdateContext AnimPlayerContext = Context.FractionalWeight(AnimPlayerBlendWeight);
		{

#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG 
			UE::Anim::TScopedGraphMessage<UE::Anim::FAnimBlendDebugScope> BlendDebugMessage(Context, Context, AnimPlayerIndex, BlendStackSize, AnimPlayer.GetDebugColor());
#endif

			// A postponed anim is considered inactive since it hasn't started blending in yet.
			const bool bIsPlayerContextActive = bLocalIsContextActive && Context.IsActive() && !AnimPlayer.IsPostponed();
			AnimPlayer.bIsActive = bIsPlayerContextActive;
			UpdateSample(bIsPlayerContextActive ? AnimPlayerContext : AnimPlayerContext.AsInactive(), AnimPlayerIndex);

			// Found our active context! All further ones should be inactive.
			bLocalIsContextActive &= !bIsPlayerContextActive;
		}
		CurrentWeightMultiplier *= (1.f - BlendInPercentage);

		++AnimPlayerIndex;

		if (CurrentWeightMultiplier < UE_KINDA_SMALL_NUMBER)
		{
			break;
		}
	}

	// AnimPlayers[AnimPlayerIndex] is the first FBlendStackAnimPlayer with a weight contribution of zero,
	// so we can discard it and all the successive AnimPlayers as well
	// NoTe that it's safe to delete all those players, because we didn't call SamplePlayer.Update_AnyThread 
	// hence not register sequence / blendspace player InternalTimeAccumulator via FAnimTickRecord(s)
	const int32 WantedAnimPlayersNum = FMath::Max(1, AnimPlayerIndex); // we save at least one FBlendStackAnimPlayer
	while (AnimPlayers.Num() > WantedAnimPlayersNum)
	{
		PopLastAnimPlayer();
	}
}

bool FAnimNode_BlendStack_Standalone::IsSampleGraphAvailableForPlayer(const int32 PlayerIndex)
{
	// If we have any sample graphs, our player has been assigned a pose link index.
	// Players with a stored pose don't need to run the graph.
	return !PerSampleGraphPoseLinks.IsEmpty() && !AnimPlayers[PlayerIndex].HasValidPoseContext();
}

bool FAnimNode_BlendStack_Standalone::EvaluateSample(FPoseContext& Output, const int32 PlayerIndex)
{
	FBlendStackAnimPlayer& SamplePlayer = AnimPlayers[PlayerIndex];
	
	if (SamplePlayer.IsPostponed())
	{
		return false;
	}

	// MaxActiveBlends == 0, means we're using inertialization. Run the the graph.
	const bool bIsSampleGraphAvailable = IsSampleGraphAvailableForPlayer(PlayerIndex);
	if (!bIsSampleGraphAvailable)
	{
		// If we have no sample graph, evaluate the player directly.
		SamplePlayer.Evaluate_AnyThread(Output);
		return true;
	}

	const int32 SampleIndex = SamplePlayer.GetPoseLinkIndex();
	UE::BlendStack::FBlendStack_SampleGraphExecutionHelper& PoseLink = SampleGraphExecutionHelpers[SampleIndex];
	PoseLink.EvaluatePlayer(Output, SamplePlayer, PerSampleGraphPoseLinks[SampleIndex]);
		
	return true;
}

void UE::BlendStack::FBlendStack_SampleGraphExecutionHelper::EvaluatePlayer(FPoseContext& Output, FBlendStackAnimPlayer& SamplePlayer, FPoseLink& SamplePoseLink)
{
	SetInputPosePlayer(SamplePlayer);

	// Make sure CacheBones has been called before evaluating.
	ConditionalCacheBones(Output, SamplePoseLink);
	// The anim player may or may not have its Evaluate_AnyThread called through the graph update. 
	SamplePoseLink.Evaluate(Output);
}

void UE::BlendStack::FBlendStack_SampleGraphExecutionHelper::ConditionalCacheBones(const FAnimationBaseContext& Context, FPoseLink& SamplePoseLink)
{
	// Only call CacheBones when needed.
	if (!CacheBoneCounter.IsSynchronized_All(Context.AnimInstanceProxy->GetCachedBonesCounter()))
	{
		// Keep track of samples that have had CacheBones called on.
		CacheBoneCounter.SynchronizeWith(Context.AnimInstanceProxy->GetCachedBonesCounter());

		FAnimationCacheBonesContext CacheBoneContext(Context.AnimInstanceProxy);
		SamplePoseLink.CacheBones(CacheBoneContext);
	}
}

void FAnimNode_BlendStack_Standalone::UpdateSample(const FAnimationUpdateContext& Context, const int32 PlayerIndex)
{
	FBlendStackAnimPlayer& SamplePlayer = AnimPlayers[PlayerIndex];

	// Advance the blend-in time regardless of whether or not the player was updated.
	SamplePlayer.UpdateWithDeltaTime(Context.GetDeltaTime(), PlayerIndex, PlayerDepthBlendInTimeMultiplier);

	if (!SamplePlayer.IsPostponed())
	{
		const bool bHasSampleGraph = IsSampleGraphAvailableForPlayer(PlayerIndex);
		if (bHasSampleGraph)
		{
			const int32 SampleIndex = SamplePlayer.GetPoseLinkIndex();
			UE::BlendStack::FBlendStack_SampleGraphExecutionHelper& ExecutionHelper = SampleGraphExecutionHelpers[SampleIndex];
			ExecutionHelper.SetInputPosePlayer(SamplePlayer);
			// The anim player may or may not have its Update_AnyThread called through the graph update.
			PerSampleGraphPoseLinks[SampleIndex].Update(Context);
		}
		else
		{
			// If we have no sample graph, update the player directly.
			SamplePlayer.Update_AnyThread(Context);
		}
	}
}

void FAnimNode_BlendStack_Standalone::CacheBonesForSample(const FAnimationCacheBonesContext& Context, const int32 PlayerIndex)
{
	FBlendStackAnimPlayer& SamplePlayer = AnimPlayers[PlayerIndex];
	const bool bHasSampleGraph = IsSampleGraphAvailableForPlayer(PlayerIndex);
	if (bHasSampleGraph)
	{
		const int32 SampleIndex = SamplePlayer.GetPoseLinkIndex();
		UE::BlendStack::FBlendStack_SampleGraphExecutionHelper& ExecutionHelper = SampleGraphExecutionHelpers[SampleIndex];
		ExecutionHelper.ConditionalCacheBones(Context, PerSampleGraphPoseLinks[SampleIndex]);
	}
}

void FAnimNode_BlendStack_Standalone::InitializeSample(const FAnimationInitializeContext& Context, FBlendStackAnimPlayer& SamplePlayer)
{
	if (SamplePlayer.GetPoseLinkIndex() != INDEX_NONE)
	{
		const int32 SampleIndex = SamplePlayer.GetPoseLinkIndex();
		FPoseLink& PoseLink = PerSampleGraphPoseLinks[SampleIndex];
		UE::BlendStack::FBlendStack_SampleGraphExecutionHelper& ExecutionHelper = SampleGraphExecutionHelpers[SampleIndex];
		ExecutionHelper.SetInputPosePlayer(SamplePlayer);
		PoseLink.Initialize(Context);
		ExecutionHelper.ConditionalCacheBones(Context, PoseLink);
	}
}

float FAnimNode_BlendStack_Standalone::GetCurrentAssetLength() const
{
	return AnimPlayers.IsEmpty() ? 0.0f : AnimPlayers[0].GetCurrentAssetLength();
}

float FAnimNode_BlendStack_Standalone::GetCurrentAssetTime() const
{
	return AnimPlayers.IsEmpty() ? 0.0f : AnimPlayers[0].GetCurrentAssetTime();
}

UAnimationAsset* FAnimNode_BlendStack_Standalone::GetAnimAsset() const
{
	return AnimPlayers.IsEmpty() ? nullptr : AnimPlayers[0].GetAnimationAsset();
}

float FAnimNode_BlendStack_Standalone::GetAccumulatedTime() const
{
	return AnimPlayers.IsEmpty() ? 0.f : AnimPlayers[0].GetAccumulatedTime();
}

bool FAnimNode_BlendStack_Standalone::GetMirror() const
{
	return AnimPlayers.IsEmpty() ? false : AnimPlayers[0].GetMirror();
}

FVector FAnimNode_BlendStack_Standalone::GetBlendParameters() const
{
	return AnimPlayers.IsEmpty() ? FVector::ZeroVector : AnimPlayers[0].GetBlendParameters();
}

static void RequestInertialBlend(const FAnimationUpdateContext& Context, float BlendTime, const UBlendProfile* BlendProfile, EAlphaBlendOption BlendOption, FName InertialBlendNodeTag)
{
#if ENABLE_ANIM_DEBUG
	if (!UE::BlendStack::GVarAnimBlendStackEnable)
	{
		return;
	}
#endif // ENABLE_ANIM_DEBUG

	if (BlendTime > 0.0f)
	{
		UE::Anim::IInertializationRequester* InertializationRequester = Context.GetMessage<UE::Anim::IInertializationRequester>();
		if (InertializationRequester)
		{
			FInertializationRequest Request;
			Request.Duration = BlendTime;
			Request.BlendProfile = BlendProfile;
			Request.bUseBlendMode = true;
			Request.BlendMode = BlendOption;
			Request.Tag = InertialBlendNodeTag;
#if ANIM_TRACE_ENABLED
			Request.NodeId = Context.GetCurrentNodeId();
			Request.AnimInstance = Context.AnimInstanceProxy->GetAnimInstanceObject();
#endif

			InertializationRequester->RequestInertialization(Request);
		}
		else
		{
			FAnimNode_Inertialization::LogRequestError(Context, Context.GetCurrentNodeId());
		}
	}
}

bool FAnimNode_BlendStack_Standalone::StitchTo(const FAnimationUpdateContext& Context, UAnimationAsset* AnimationAsset, float AccumulatedTime, bool bLoop,
	bool bMirrored, UMirrorDataTable* MirrorDataTable, float BlendTime, const UBlendProfile* BlendProfile, EAlphaBlendOption BlendOption, bool bUseInertialBlend, FName InertialBlendNodeTag,
	const FVector& BlendParameters, float PlayRate, float ActivationDelay)
{
	using namespace UE::Anim;

	if (!StitchDatabase)
	{
		return false;
	}

	if (MaxActiveBlends <= 0)
	{
		UE_LOGF(LogBlendStack, Error, "FAnimNode_BlendStack_Standalone::BlendTo cannot use StitchDatabase '%ls', since MaxActiveBlends should be at least 1. Defaulting to regular blend", *GetNameSafe(StitchDatabase));
		return false;
	}
		
	IPoseSearchProvider* PoseSearchProvider = IPoseSearchProvider::Get();
	if (!PoseSearchProvider)
	{
		UE_LOGF(LogBlendStack, Error, "FAnimNode_BlendStack_Standalone::BlendTo cannot use StitchDatabase '%ls', because of missing IPoseSearchProvider (is PoseSearch plugin enabled?). Defaulting to regular blend", *GetNameSafe(StitchDatabase));
		return false;
	}

	// looking for an animation stitch from the StitchDatabase that will connect, in BlendTime seconds, 
	// the currently playing animation pose to the pose from AnimationAsset at AccumulatedTime + BlendTime
	const UObject* AssetToSearch = StitchDatabase.Get();

	IPoseSearchProvider::FSearchPlayingAsset PlayingAsset;
	PlayingAsset.Asset = GetAnimAsset();
	PlayingAsset.AccumulatedTime = GetAccumulatedTime();
	PlayingAsset.bMirrored = GetMirror();
	PlayingAsset.BlendParameters = GetBlendParameters();

	IPoseSearchProvider::FSearchFutureAsset FutureAsset;
	FutureAsset.Asset = AnimationAsset;
	FutureAsset.AccumulatedTime = AccumulatedTime + BlendTime;
	FutureAsset.IntervalTime = BlendTime;

	const IPoseSearchProvider::FSearchResult SearchResult = PoseSearchProvider->Search(Context, MakeArrayView(&AssetToSearch, 1), PlayingAsset, FutureAsset);
	UAnimationAsset* StitchAnimationAsset = Cast<UAnimationAsset>(SearchResult.SelectedAsset);
	if (!StitchAnimationAsset)
	{
		UE_LOGF(LogBlendStack, Error, "FAnimNode_BlendStack_Standalone::BlendTo cannot use StitchDatabase '%ls', because of missing IPoseSearchProvider::Search couldn't select a StitchAnimationAsset. Defaulting to regular blend", *GetNameSafe(StitchDatabase));
		return false;
	}

	if (SearchResult.Dissimilarity > StitchBlendMaxCost)
	{
		UE_LOGF(LogBlendStack, Display, "FAnimNode_BlendStack_Standalone::BlendTo StitchDatabase '%ls' search cost is %f, above StitchBlendMaxCost %f. Defaulting to regular blend", *GetNameSafe(StitchDatabase), SearchResult.Dissimilarity, StitchBlendMaxCost);
		return false;
	}

	// blend to the selected animation stitch
	InternalBlendTo(Context, StitchAnimationAsset, SearchResult.TimeOffsetSeconds, false, SearchResult.bMirrored, MirrorDataTable,
		StitchBlendTime, BlendProfile, BlendOption, bUseInertialBlend, InertialBlendNodeTag, BlendParameters, SearchResult.WantedPlayRate, ActivationDelay,
		NAME_None, EAnimGroupRole::CanBeLeader, EAnimSyncMethod::DoNotSync, false, NAME_None, EAnimGroupRole::CanBeLeader, EAnimSyncMethod::DoNotSync);

	// blend with an ActivationDelay of BlendTime - StitchBlendTime + ActivationDelay seconds
	// to the AnimationAsset at AccumulatedTime + BlendTime - StitchBlendTime seconds in the future,
	// so at BlendTime seconds ahead the AnimationAsset is playing the fully blended in pose at AccumulatedTime + BlendTime
	InternalBlendTo(Context, AnimationAsset, AccumulatedTime + BlendTime - StitchBlendTime, bLoop, bMirrored, MirrorDataTable,
		StitchBlendTime, BlendProfile, BlendOption, bUseInertialBlend, InertialBlendNodeTag, BlendParameters, PlayRate, BlendTime - StitchBlendTime + ActivationDelay,
		NAME_None, EAnimGroupRole::CanBeLeader, EAnimSyncMethod::DoNotSync, false, NAME_None, EAnimGroupRole::CanBeLeader, EAnimSyncMethod::DoNotSync);

	return true;
}

void FAnimNode_BlendStack_Standalone::BlendTo(const FAnimationUpdateContext& Context, UAnimationAsset* AnimationAsset, float AccumulatedTime, bool bLoop,
	bool bMirrored, UMirrorDataTable* MirrorDataTable, float BlendTime, const UBlendProfile* BlendProfile, EAlphaBlendOption BlendOption, bool bUseInertialBlend, FName InertialBlendNodeTag,
	const FVector& BlendParameters, float PlayRate, float ActivationDelay, FName GroupName, EAnimGroupRole::Type GroupRole, EAnimSyncMethod GroupMethod, bool bOverridePositionWhenJoiningSyncGroupAsLeader,
	FName PrevGroupName, EAnimGroupRole::Type PrevGroupRole, EAnimSyncMethod PrevMethod)
{
	if (!StitchTo(Context, AnimationAsset, AccumulatedTime, bLoop, bMirrored, MirrorDataTable, BlendTime, 
		BlendProfile, BlendOption, bUseInertialBlend, InertialBlendNodeTag, BlendParameters, PlayRate, ActivationDelay))
	{
		InternalBlendTo(Context, AnimationAsset, AccumulatedTime, bLoop, bMirrored, MirrorDataTable,
			BlendTime, BlendProfile, BlendOption, bUseInertialBlend, InertialBlendNodeTag, BlendParameters, PlayRate, ActivationDelay, 
			GroupName, GroupRole, GroupMethod, bOverridePositionWhenJoiningSyncGroupAsLeader, PrevGroupName, PrevGroupRole, PrevMethod);
	}
}

void FAnimNode_BlendStack_Standalone::InternalBlendTo(const FAnimationUpdateContext& Context, UAnimationAsset* AnimationAsset, float AccumulatedTime, bool bLoop,
	bool bMirrored, UMirrorDataTable* MirrorDataTable, float BlendTime, const UBlendProfile* BlendProfile, EAlphaBlendOption BlendOption, bool bUseInertialBlend, FName InertialBlendNodeTag,
	const FVector& BlendParameters, float PlayRate, float ActivationDelay, FName GroupName, EAnimGroupRole::Type GroupRole, EAnimSyncMethod GroupMethod, bool bOverridePositionWhenJoiningSyncGroupAsLeader,
	FName PrevGroupName, EAnimGroupRole::Type PrevGroupRole, EAnimSyncMethod PrevGroupMethod)
{
	const bool bBlendStackIsEmpty = AnimPlayers.IsEmpty();

	// If the blend stack is empty, we shouldn't blend. Pop into the requested pose.
	if (bBlendStackIsEmpty)
	{		
		BlendTime = 0.0f;
	}
#if WITH_EDITOR
	else
	{
		// returns AnimationAsset EAdditiveAnimationType. AAT_MAX if AnimationAsset is a UBlendSpace with inconsistent EAdditiveAnimationType
		auto GetAdditiveAnimType = [](UAnimationAsset* AnimationAsset) -> EAdditiveAnimationType
			{
				EAdditiveAnimationType AdditiveAnimationType = AAT_None;
				if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(AnimationAsset))
				{
					AdditiveAnimationType = AnimSequenceBase->GetAdditiveAnimType();
				}
				else if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAsset))
				{
					AdditiveAnimationType = AAT_MAX;
					for (const FBlendSample& BlendSample : BlendSpace->GetBlendSamples())
					{
						if (AdditiveAnimationType == AAT_MAX)
						{
							AdditiveAnimationType = BlendSample.Animation->GetAdditiveAnimType();
						}
						else if (AdditiveAnimationType != BlendSample.Animation->GetAdditiveAnimType())
						{
							AdditiveAnimationType = AAT_MAX;
							break;
						}
					}
				}

				return AdditiveAnimationType;
			};

		const EAdditiveAnimationType OldAdditiveAnimationType = GetAdditiveAnimType(AnimPlayers[0].GetAnimationAsset());
		const EAdditiveAnimationType NewAdditiveAnimationType = GetAdditiveAnimType(AnimationAsset);
		if (NewAdditiveAnimationType != OldAdditiveAnimationType)
		{
			UE_LOGF(LogBlendStack, Error, "FAnimNode_BlendStack_Standalone::InternalBlendTo AdditiveAnimType mismatch between input animation '%ls' and currently playing animation '%ls'", *GetNameSafe(AnimationAsset), *GetNameSafe(AnimPlayers[0].GetAnimationAsset()));
		}
	}
#endif // WITH_EDITOR

	if (bUseInertialBlend)
	{
		RequestInertialBlend(Context, BlendTime, BlendProfile, BlendOption, InertialBlendNodeTag);
		BlendTime = 0.0f;
	}
	
	int32 NewSamplePoseLinkIndex = CurrentSamplePoseLink;
	if (!bBlendStackIsEmpty && AnimPlayers[0].IsPostponed())
	{
		// we allow only one player with TimeToActivation > 0:
		// replacing AnimPlayers[0] with this new BlendTo request
	}
	// If we don't add a new player, re-use the same graph...
	else if (!bBlendStackIsEmpty && AnimPlayers[0].GetBlendInPercentage() < 1.f &&
		AnimPlayers[0].GetCurrentBlendInTime() < MaxBlendInTimeToOverrideAnimation)
	{
		// replacing AnimPlayers[0] with this new BlendTo request
		UE_LOGF(LogBlendStack, Verbose, "FAnimNode_BlendStack_Standalone::InternalBlendTo '%ls' replaced by '%ls' because blend time in is less than MaxBlendInTimeToOverrideAnimation (%.2f / %.2f)", *AnimPlayers[0].GetAnimationName(), *GetNameSafe(AnimationAsset), AnimPlayers[0].GetCurrentBlendInTime(), MaxBlendInTimeToOverrideAnimation);
	}
	else if (AnimPlayers.Num() <= MaxActiveBlends + 2)
	{
#if !NO_LOGGING
		if (!bBlendStackIsEmpty)
		{
			const float SecondsFromEnd = AnimPlayers[0].GetSecondsFromEnd();
			if (SecondsFromEnd < BlendTime)
			{
				UE_LOGF(LogBlendStack, Verbose, "FAnimNode_BlendStack_Standalone::InternalBlendTo remaining playback time for '%ls' is only %f sec, while the requested blend time is %f sec. This will result in a poor quality blend, since the currently playing asset will end and freeze its pose, curves, and attributes before the new animation is fully relevant", *AnimPlayers[0].GetAnimationName(), SecondsFromEnd, BlendTime);
			}
		}
#endif // !NO_LOGGING

		AnimPlayers.Insert(FBlendStackAnimPlayer(), 0);
		// ...otherwise, assign a new graph.
		NewSamplePoseLinkIndex = GetNextPoseLinkIndex();
	}
	else
	{
#if !NO_LOGGING
		if (AnimPlayers.Num() > 1)
		{
			const float SecondsFromEnd = AnimPlayers[1].GetSecondsFromEnd();
			if (SecondsFromEnd < BlendTime)
			{
				UE_LOGF(LogBlendStack, Verbose, "FAnimNode_BlendStack_Standalone::InternalBlendTo remaining playback time for '%ls' is only %f sec, while the requested blend time is %f sec. This will result in a poor quality blend, since the currently playing asset will end and freeze its pose, curves, and attributes before the new animation is fully relevant", *AnimPlayers[0].GetAnimationName(), SecondsFromEnd, BlendTime);
			}
		}
#endif // !NO_LOGGING

		// else it means we had multiple BlendTo suring the same frame. we'll let the last one win
		UE_LOGF(LogBlendStack, Warning, "FAnimNode_BlendStack_Standalone::InternalBlendTo multiple BlendTo requests during the same frame: only the last request will be put on this BlendStack");
	}

	if (AnimPlayers.Num() > 1)
	{
		FBlendStackAnimPlayer& PrevMainAnimPlayer = AnimPlayers[1];
		if (FAnimNode_AssetPlayerBase* PrevAssetPlayerNode = PrevMainAnimPlayer.GetAssetPlayerNode())
		{
			PrevAssetPlayerNode->SetGroupMethod(PrevGroupMethod);
			PrevAssetPlayerNode->SetGroupRole(PrevGroupRole);
			PrevAssetPlayerNode->SetGroupName(PrevGroupName);
		}

		// Clear up any sync group of other AnimPlayers
		for (int32 i = 2; i < AnimPlayers.Num(); ++i)
		{
			FBlendStackAnimPlayer& AnimPlayer = AnimPlayers[i];
			if (FAnimNode_AssetPlayerBase* AssetPlayerNode = AnimPlayer.GetAssetPlayerNode())
			{
				AssetPlayerNode->SetGroupMethod(EAnimSyncMethod::DoNotSync);
				AssetPlayerNode->SetGroupRole(EAnimGroupRole::CanBeLeader);
				AssetPlayerNode->SetGroupName(NAME_None);
			}
		}
	}

	FBlendStackAnimPlayer& MainAnimPlayer = AnimPlayers[0];

	FAnimationInitializeContext InitContext(Context.AnimInstanceProxy, Context.SharedContext);
	MainAnimPlayer.Initialize(InitContext, AnimationAsset, AccumulatedTime, bLoop, bMirrored, MirrorDataTable, BlendTime, BlendProfile, BlendOption, BlendParameters, PlayRate, ActivationDelay, NewSamplePoseLinkIndex, GroupName, GroupRole, GroupMethod, bOverridePositionWhenJoiningSyncGroupAsLeader);
	InitializeSample(InitContext, MainAnimPlayer);

	LastBlendToFrameCounter = GFrameCounter;
}

void FAnimNode_BlendStack_Standalone::Reset()
{
	LastBlendToFrameCounter = GFrameCounter;

	// reserving MaxActiveBlends + 2 AnimPlayers, to avoid any reallocation
	AnimPlayers.Reserve(MaxActiveBlends + 2);
	AnimPlayers.Reset();

	if (bShouldFilterNotifies)
	{
		NotifiesFiredLastTick->Reset();
		NotifyRecencyMap->Reset();
	}
}

int32 FAnimNode_BlendStack_Standalone::GetNextPoseLinkIndex()
{
	if (PerSampleGraphPoseLinks.IsEmpty())
	{
		return INDEX_NONE;
	}

	const int32 NumPoseLinks = PerSampleGraphPoseLinks.Num();
	++CurrentSamplePoseLink;
	if (CurrentSamplePoseLink == NumPoseLinks) { CurrentSamplePoseLink = 0; }

	return CurrentSamplePoseLink;
}

bool FAnimNode_BlendStack_Standalone::AnyNewBlendToThisFrame() const
{
	return LastBlendToFrameCounter == GFrameCounter;
}

void FAnimNode_BlendStack_Standalone::UpdatePlayRate(float PlayRate)
{
	if (!AnimPlayers.IsEmpty())
	{
		AnimPlayers[0].UpdatePlayRate(PlayRate);
	}
}

void FAnimNode_BlendStack_Standalone::GatherDebugData(FNodeDebugData& DebugData)
{
#if ENABLE_ANIM_DEBUG
	DebugData.AddDebugItem(FString::Printf(TEXT("%s"), *DebugData.GetNodeName(this)));
	for (int32 i = 0; i < AnimPlayers.Num(); ++i)
	{
		const FBlendStackAnimPlayer& AnimPlayer = AnimPlayers[i];
		DebugData.AddDebugItem(FString::Printf(TEXT("%d) t:%.2f/%.2f a:%.2f m:%d %s"),
				i, AnimPlayer.GetCurrentBlendInTime(), AnimPlayer.GetTotalBlendInTime(), AnimPlayer.GetTimeToActivation(),
				AnimPlayer.GetMirror() ? 1 : 0, *AnimPlayer.GetAnimationName()));
	}
#endif // ENABLE_ANIM_DEBUG

	// propagating GatherDebugData to the AnimPlayers
	for (FBlendStackAnimPlayer& AnimPlayer : AnimPlayers)
	{
		AnimPlayer.GetMirrorNode().GatherDebugData(DebugData);
	}
}

// this function is the optimized version of 
// FAnimationRuntime::BlendTwoPosesTogetherPerBone(InOutPoseData, OtherPoseData, OtherPoseWeights, InOutPoseData);
void FAnimNode_BlendStack_Standalone::BlendWithPosePerBone(FAnimationPoseData& InOutPoseData, const FAnimationPoseData& OtherPoseData, TConstArrayView<float> OtherPoseWeights)
{
	using namespace UE::Anim;
	using namespace UE::BlendStack;

	FCompactPose& InOutPose = InOutPoseData.GetPose();
	FBlendedCurve& InOutCurve = InOutPoseData.GetCurve();
	FStackAttributeContainer& InOutAttributes = InOutPoseData.GetAttributes();

	const FCompactPose& OtherPose = OtherPoseData.GetPose();

	for (FCompactPoseBoneIndex BoneIndex : InOutPose.ForEachBoneIndex())
	{
		const float OtherPoseBoneWeight = OtherPoseWeights[BoneIndex.GetInt()];
		if (FAnimationRuntime::IsFullWeight(OtherPoseBoneWeight))
		{
			InOutPose[BoneIndex] = OtherPose[BoneIndex];
		}
		else if (FAnimationRuntime::HasWeight(OtherPoseBoneWeight))
		{
			const ScalarRegister VInOutPoseBoneWeight(1.f - OtherPoseBoneWeight);
			const ScalarRegister VOtherPoseBoneWeight(OtherPoseBoneWeight);

			InOutPose[BoneIndex] *= VInOutPoseBoneWeight;
			InOutPose[BoneIndex].AccumulateWithShortestRotation(OtherPose[BoneIndex], VOtherPoseBoneWeight);
		}
		// else we leave InOutPose[BoneIndex] as is
	}

	// Ensure that all of the resulting rotations are normalized
	InOutPose.NormalizeRotations();

	FBlendedCurveUtils::LerpToPerBoneEx(InOutCurve, OtherPoseData.GetCurve(), InOutPoseData.GetPose().GetBoneContainer(), OtherPoseWeights);

	// @todo: optimize away the copy
	FStackAttributeContainer CustomAttributes;
	Attributes::BlendAttributesPerBone(InOutAttributes, OtherPoseData.GetAttributes(), OtherPoseWeights, CustomAttributes);
	InOutAttributes = CustomAttributes;
}

// this function is the optimized version of 
// FAnimationRuntime::BlendTwoPosesTogether(InOutPoseData, OtherPoseData, InOutPoseWeight, InOutPoseData);
void FAnimNode_BlendStack_Standalone::BlendWithPose(FAnimationPoseData& InOutPoseData, const FAnimationPoseData& OtherPoseData, const float InOutPoseWeight)
{
	using namespace UE::Anim;
	using namespace UE::BlendStack;

	FCompactPose& InOutPose = InOutPoseData.GetPose();
	FBlendedCurve& InOutCurve = InOutPoseData.GetCurve();
	FStackAttributeContainer& InOutAttributes = InOutPoseData.GetAttributes();

	const FCompactPose& OtherPose = OtherPoseData.GetPose();

	const float OtherPoseWeight = 1.f - InOutPoseWeight;

	// @todo: reimplement the ispc version of this if needed
	const ScalarRegister VInOutPoseWeight(InOutPoseWeight);
	const ScalarRegister VOtherPoseWeight(OtherPoseWeight);

	for (FCompactPoseBoneIndex BoneIndex : InOutPose.ForEachBoneIndex())
	{
		InOutPose[BoneIndex] *= VInOutPoseWeight;
		InOutPose[BoneIndex].AccumulateWithShortestRotation(OtherPose[BoneIndex], VOtherPoseWeight);
	}

	// Ensure that all of the resulting rotations are normalized
	InOutPose.NormalizeRotations();

	FBlendedCurveUtils::LerpToEx(InOutCurve, OtherPoseData.GetCurve(), OtherPoseWeight);
	
	// @todo: optimize away the copy
	FStackAttributeContainer CustomAttributes;
	Attributes::BlendAttributes({ InOutAttributes, OtherPoseData.GetAttributes() }, { InOutPoseWeight, OtherPoseWeight }, { 0, 1 }, CustomAttributes);
	InOutAttributes = CustomAttributes;
}

/////////////////////////////////////////////////////
// FAnimNode_BlendStack

bool FAnimNode_BlendStack::NeedsReset(const FAnimationUpdateContext& Context) const
{
	const bool bNeedsReset =
		bResetOnBecomingRelevant &&
		UpdateCounter.HasEverBeenUpdated() &&
		!UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter());
	return bNeedsReset;
}

bool FAnimNode_BlendStack::ConditionalBlendTo(const FAnimationUpdateContext& Context)
{
	bool bExecuteBlendTo = false;
	if (AnimationAsset == nullptr && !bForceBlendNextUpdate)
	{
		bExecuteBlendTo = false;
	}
	else if (AnimPlayers.IsEmpty())
	{
		bExecuteBlendTo = true;
	}
	else
	{
		const FBlendStackAnimPlayer& MainAnimPlayer = AnimPlayers[0];
		const UAnimationAsset* PlayingAnimationAsset = MainAnimPlayer.GetAnimationAsset();

		if (bForceBlendNextUpdate)
		{
			bForceBlendNextUpdate = false;
			bExecuteBlendTo = true;
		}
		else if (AnimationAsset != PlayingAnimationAsset)
		{
			bExecuteBlendTo = true;
		}
		else if (bMirrored != MainAnimPlayer.GetMirror())
		{
			bExecuteBlendTo = true;
		}
		else if ((BlendParameters - MainAnimPlayer.GetBlendParameters()).SizeSquared() > FMath::Square(BlendParametersDeltaThreshold))
		{
			bExecuteBlendTo = true;
		}
		else if (MaxAnimationDeltaTime >= 0.f && FMath::Abs(AnimationTime - MainAnimPlayer.GetAccumulatedTime()) > MaxAnimationDeltaTime)
		{
			bExecuteBlendTo = true;
		}
	}

	if (bExecuteBlendTo)
	{
		BlendTo(Context, AnimationAsset, AnimationTime, bLoop, bMirrored, MirrorDataTable.Get(),
			BlendTime, BlendProfile, BlendOption, bUseInertialBlend, InertialBlendNodeTag, BlendParameters, WantedPlayRate, ActivationDelayTime,
			GetGroupName(), GetGroupRole(), GetGroupMethod());
	}

	return bExecuteBlendTo;
}

void FAnimNode_BlendStack::Reset()
{
	Super::Reset();
	bForceBlendNextUpdate = false;
}

void FAnimNode_BlendStack::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	if (NeedsReset(Context))
	{
		Reset();
	}

	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	GetEvaluateGraphExposedInputs().Execute(Context);

	const bool bExecuteBlendTo = ConditionalBlendTo(Context);
	const bool bDidBlendToRequestAnInertialBlend = bExecuteBlendTo && bUseInertialBlend;
	UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimInertializationSyncScope> InertializationSync(bDidBlendToRequestAnInertialBlend, Context);
	
	UpdatePlayRate(WantedPlayRate);
	UpdateBlendspaceParameters(BlendspaceUpdateMode, BlendParameters);

	if (bForceBlendTo && !AnyNewBlendToThisFrame())
	{
		check(!AnimPlayers.IsEmpty());
		const FBlendStackAnimPlayer& MainAnimPlayer = AnimPlayers[0];

		// blending into the new main anim player with the new integrated time to avoid pops on this discontinuity
		InternalBlendTo(Context, MainAnimPlayer.GetAnimationAsset(), MainAnimPlayer.GetAccumulatedTime(), MainAnimPlayer.IsLooping(), MainAnimPlayer.GetMirror(),
			MainAnimPlayer.GetMirrorDataTable(), BlendTime,
			MainAnimPlayer.GetBlendProfile(), MainAnimPlayer.GetBlendOption(), bUseInertialBlend, 
			InertialBlendNodeTag, MainAnimPlayer.GetBlendParameters(), MainAnimPlayer.GetPlayRate(), 0.f,
			GetGroupName(), GetGroupRole(), GetGroupMethod(), GetOverridePositionWhenJoiningSyncGroupAsLeader(),
			GetPrevGroupName(), GetPrevGroupRole(), GetPrevGroupMethod());
	}

	Super::UpdateAssetPlayer(Context);
}

void FAnimNode_BlendStack::ForceBlendNextUpdate()
{
	bForceBlendNextUpdate = true;
}

void UE::BlendStack::FBlendStack_SampleGraphExecutionHelper::SetInputPosePlayer(FBlendStackAnimPlayer& InPlayer)
{
	// Because our anim players may get reallocated, or change indices due to push/pops,
	// we must call this before every operation that might end up needing the anim player through the graph's input nodes.
	Player = &InPlayer;
}

FName FAnimNode_BlendStack::GetGroupName() const
{
	return GET_ANIM_NODE_DATA(FName, GroupName);
}

EAnimGroupRole::Type FAnimNode_BlendStack::GetGroupRole() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<EAnimGroupRole::Type>, GroupRole);
}

EAnimSyncMethod FAnimNode_BlendStack::GetGroupMethod() const
{
	return GET_ANIM_NODE_DATA(EAnimSyncMethod, Method);
}

FName FAnimNode_BlendStack::GetPrevGroupName() const
{
	return GET_ANIM_NODE_DATA(FName, PrevGroupName);
}

EAnimGroupRole::Type FAnimNode_BlendStack::GetPrevGroupRole() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<EAnimGroupRole::Type>, PrevGroupRole);
}

EAnimSyncMethod FAnimNode_BlendStack::GetPrevGroupMethod() const
{
	return GET_ANIM_NODE_DATA(EAnimSyncMethod, PrevMethod);
}

bool FAnimNode_BlendStack::GetIgnoreForRelevancyTest() const
{
	return GET_ANIM_NODE_DATA(bool, bIgnoreForRelevancyTest);
}

bool FAnimNode_BlendStack::SetGroupName(FName InGroupName)
{
#if WITH_EDITORONLY_DATA
	GroupName = InGroupName;
#endif

	if(FName* GroupNamePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(FName, GroupName))
	{
		*GroupNamePtr = InGroupName;
		return true;
	}

	return false;
}

bool FAnimNode_BlendStack::SetGroupRole(EAnimGroupRole::Type InRole)
{
#if WITH_EDITORONLY_DATA
	GroupRole = InRole;
#endif
	
	if(TEnumAsByte<EAnimGroupRole::Type>* GroupRolePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(TEnumAsByte<EAnimGroupRole::Type>, GroupRole))
	{
		*GroupRolePtr = InRole;
		return true;
	}

	return false;
}

bool FAnimNode_BlendStack::SetGroupMethod(EAnimSyncMethod InMethod)
{
#if WITH_EDITORONLY_DATA
	Method = InMethod;
#endif

	if(EAnimSyncMethod* MethodPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(EAnimSyncMethod, Method))
	{
		*MethodPtr = InMethod;
		return true;
	}

	return false;
}

bool FAnimNode_BlendStack::IsLooping() const
{
	if (!AnimPlayers.IsEmpty())
	{
		return AnimPlayers[0].IsLooping();
	}
	return false;
}


bool FAnimNode_BlendStack::SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest)
{
#if WITH_EDITORONLY_DATA
	bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest;
#endif

	if (bool* bIgnoreForRelevancyTestPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bIgnoreForRelevancyTest))
	{
		*bIgnoreForRelevancyTestPtr = bInIgnoreForRelevancyTest;
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
