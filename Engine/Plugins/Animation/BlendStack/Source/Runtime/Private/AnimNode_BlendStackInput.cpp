// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendStack/AnimNode_BlendStackInput.h"
#include "BlendStack/AnimNode_BlendStack.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendStackInput)

void FAnimNode_BlendStackInput::Evaluate_AnyThread(FPoseContext& Output)
{
	check(Player && *Player);
	(*Player)->Evaluate_AnyThread(Output);
}

static FAnimNode_BlendStack_Standalone* GetBlendStackNodeFromIndex(const FAnimationBaseContext& Context, IAnimClassInterface* AnimBlueprintClass, int32 NodeIndex)
{
	const TArray<FStructProperty*>& AnimNodeProperties = AnimBlueprintClass->GetAnimNodeProperties();
	check(AnimNodeProperties.IsValidIndex(NodeIndex));

	FStructProperty* LinkedProperty = AnimNodeProperties[NodeIndex];
	return LinkedProperty->ContainerPtrToValuePtr<FAnimNode_BlendStack_Standalone>(Context.AnimInstanceProxy->GetAnimInstanceObject());
}


void FAnimNode_BlendStackInput::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	GetEvaluateGraphExposedInputs().Execute(Context);

	// If there's no player, use the allocated blend stack index to get a reference to it.
	// This should only happen on the first ever update of this node.
	if (Player == nullptr)
	{
		IAnimClassInterface* AnimBlueprintClass = Context.GetAnimClass();
		check(AnimBlueprintClass);
		// Allocation index is the inverse of the node index.
		const int32 BlendStackNodeIndex = AnimBlueprintClass->GetAnimNodeProperties().Num() - 1 - BlendStackAllocationIndex;
		check(Context.AnimInstanceProxy);
		FAnimNode_BlendStack_Standalone* BlendStackNode = GetBlendStackNodeFromIndex(Context, AnimBlueprintClass, BlendStackNodeIndex);
		Player = &BlendStackNode->SampleGraphExecutionHelpers[SampleIndex].Player;
	}

	check(Player && *Player)

	if (bOverridePlayRate)
	{
		(*Player)->UpdatePlayRate(PlayRate);
	}
	
	(*Player)->Update_AnyThread(Context);
}


