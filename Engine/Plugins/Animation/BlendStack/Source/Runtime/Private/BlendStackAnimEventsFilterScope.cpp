// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendStackAnimEventsFilterScope.h"

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::Anim::FBlendStackAnimEventsFilterScope)

namespace UE { namespace Anim {
	
	FBlendStackAnimEventsFilterContext::FBlendStackAnimEventsFilterContext(const TSharedRef<TArray<FName>> & InFiredNotifies, const TSharedRef<TMap<FName, float>>& InNotifyBanList)
		: FiredNotifies(InFiredNotifies), NotifyBanList(InNotifyBanList)
	{
	}

	bool FBlendStackAnimEventsFilterContext::ShouldFilterNotify(const FAnimNotifyEventReference& InNotifyEventRef) const
	{
		const FAnimNotifyEvent* Notify = InNotifyEventRef.GetNotify();

		// Only filter anim notifies
		if (!Notify || Notify->NotifyStateClass)
		{
			return false;
		}
		
		const FName NotifyName = Notify->NotifyName;
		const bool bWasRecentlyFired = NotifyBanList->Contains(NotifyName);
		const bool bWasAlreadyFiredThisTick = FiredNotifies->Contains(NotifyName);
	
		if (!bWasAlreadyFiredThisTick && !bWasRecentlyFired)
		{
			FiredNotifies->Push(NotifyName);
			return false;
		}
		
		return true;
	}

	FBlendStackAnimEventsFilterScope::FBlendStackAnimEventsFilterScope(const TSharedRef<TArray<FName>> & InFiredNotifies, const TSharedRef<TMap<FName, float>> & InNotifyBanList)
		: FiredNotifies(InFiredNotifies), NotifyBanList(InNotifyBanList)
	{
	}

	FBlendStackAnimEventsFilterScope::FBlendStackAnimEventsFilterScope(const TSharedPtr<TArray<FName>> & InFiredNotifies, const TSharedPtr<TMap<FName,float>> & InNotifyBanList)
		: FiredNotifies(InFiredNotifies.ToSharedRef()), NotifyBanList(InNotifyBanList.ToSharedRef())
	{
	}
	
	TUniquePtr<const IAnimNotifyEventContextDataInterface> FBlendStackAnimEventsFilterScope::MakeUniqueEventContextData() const
	{
		return MakeUnique<const FBlendStackAnimEventsFilterContext>(FiredNotifies, NotifyBanList);
	}
}}	// namespace UE::Anim