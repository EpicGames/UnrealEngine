// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimEventsFilterScope.h"
#include "Animation/AnimNodeMessages.h"
#include "Animation/AnimationAsset.h"

namespace UE { namespace Anim {
	
	class FBlendStackAnimEventsFilterContext : public IAnimEventsFilterContext
	{
	public:
		FBlendStackAnimEventsFilterContext() {}
		FBlendStackAnimEventsFilterContext(const TSharedRef<TArray<FName>> & InFiredNotifies, const TSharedRef<TMap<FName,float>>& InNotifyBanList);
		
		virtual bool ShouldFilterNotify(const FAnimNotifyEventReference& InNotifyEventRef) const override;
	
	private:
		// @todo: add inline allocator with some preallocated memory
		TSharedRef<TArray<FName>> FiredNotifies;
		// @todo: add inline allocator with some preallocated memory
		TSharedRef<TMap<FName,float>> NotifyBanList;
	};
	
	class FBlendStackAnimEventsFilterScope : public IGraphMessage
	{
		DECLARE_ANIMGRAPH_MESSAGE(FBlendStackAnimEventsFilterScope);

	public:
		FBlendStackAnimEventsFilterScope(const TSharedRef<TArray<FName>> & InFiredNotifies, const TSharedRef<TMap<FName,float>> & InNotifyBanList);
		FBlendStackAnimEventsFilterScope(const TSharedPtr<TArray<FName>> & InFiredNotifies, const TSharedPtr<TMap<FName,float>> & InNotifyBanList);
		
		virtual TUniquePtr<const IAnimNotifyEventContextDataInterface> MakeUniqueEventContextData() const override;

	private:
		// @todo: add inline allocator with some preallocated memory
		TSharedRef<TArray<FName>> FiredNotifies;
		// @todo: add inline allocator with some preallocated memory
		TSharedRef<TMap<FName,float>> NotifyBanList;
	};
	
}}	// namespace UE::Anim
