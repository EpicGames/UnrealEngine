// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendStack/BlendStackAnimNodeLibrary.h"
#include "BlendStack/AnimNode_BlendStack.h"
#include "BlendStack/AnimNode_BlendStackInput.h"
#include "BlendStack/BlendStackDefines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendStackAnimNodeLibrary)

FBlendStackAnimNodeReference UBlendStackAnimNodeLibrary::ConvertToBlendStackNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FBlendStackAnimNodeReference>(Node, Result);
}

UAnimationAsset* UBlendStackAnimNodeLibrary::GetCurrentBlendStackAnimAsset(const FAnimNodeReference& Node)
{
	if (FAnimNode_BlendStackInput* BlendStackInput = Node.GetAnimNodePtr<FAnimNode_BlendStackInput>())
	{
		if (BlendStackInput->Player && *BlendStackInput->Player )
		{
			return (*BlendStackInput->Player)->GetAnimationAsset();
		}
	}
	return nullptr;
}

float UBlendStackAnimNodeLibrary::GetCurrentBlendStackAnimAssetTime(const FAnimNodeReference& Node)
{
	if (FAnimNode_BlendStackInput* BlendStackInput = Node.GetAnimNodePtr<FAnimNode_BlendStackInput>())
	{
		if (BlendStackInput->Player && *BlendStackInput->Player )
		{
			return (*BlendStackInput->Player)->GetAccumulatedTime();
		}
	}
	return 0.f;
}

bool UBlendStackAnimNodeLibrary::GetCurrentBlendStackAnimIsActive(const FAnimNodeReference& Node)
{
	if (FAnimNode_BlendStackInput* BlendStackInput = Node.GetAnimNodePtr<FAnimNode_BlendStackInput>())
	{
		if (BlendStackInput->Player && *BlendStackInput->Player)
		{
			return (*BlendStackInput->Player)->IsActive();
		}
	}
	return false;
}

bool UBlendStackAnimNodeLibrary::GetCurrentBlendStackAnimAssetMirrored(const FAnimNodeReference& Node)
{
	if (FAnimNode_BlendStackInput* BlendStackInput = Node.GetAnimNodePtr<FAnimNode_BlendStackInput>())
	{
		if (BlendStackInput->Player && *BlendStackInput->Player)
		{
			return (*BlendStackInput->Player)->GetMirror();
		}
	}
	return false;
}

UMirrorDataTable* UBlendStackAnimNodeLibrary::GetCurrentBlendStackAnimAssetMirrorTable(const FAnimNodeReference& Node)
{
	if (FAnimNode_BlendStackInput* BlendStackInput = Node.GetAnimNodePtr<FAnimNode_BlendStackInput>())
	{
		if (BlendStackInput->Player && *BlendStackInput->Player)
		{
			return (*BlendStackInput->Player)->GetMirrorNode().GetMirrorDataTable();
		}
	}
	return nullptr;
}

void UBlendStackAnimNodeLibrary::BlendTo(const FAnimUpdateContext& Context, 
										const FBlendStackAnimNodeReference& BlendStackNode, 
										UAnimationAsset* AnimationAsset,
										float AnimationTime,
										bool bLoop,
										bool bMirrored,
										float BlendTime,
										FVector BlendParameters,
										float WantedPlayRate,
										float ActivationDelay)
{
	if (AnimationAsset != nullptr)
	{
		if (FAnimNode_BlendStack* BlendStackNodePtr = BlendStackNode.GetAnimNodePtr<FAnimNode_BlendStack>())
		{
			if (const FAnimationUpdateContext* AnimationUpdateContext = Context.GetContext())
			{
				BlendStackNodePtr->BlendTo(
					*AnimationUpdateContext,
					AnimationAsset,
					AnimationTime,
					bLoop,
					bMirrored,
					BlendStackNodePtr->MirrorDataTable,
					BlendTime,
					BlendStackNodePtr->BlendProfile,
					BlendStackNodePtr->BlendOption,
					BlendStackNodePtr->bUseInertialBlend,
					BlendStackNodePtr->InertialBlendNodeTag,
					BlendParameters,
					WantedPlayRate,
					ActivationDelay,
					BlendStackNodePtr->GetGroupName(), BlendStackNodePtr->GetGroupRole(), BlendStackNodePtr->GetGroupMethod());
			}
			else
			{
				UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::BlendTo called with an invalid context.");
			}
		}
		else
		{
			UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::BlendTo called with an invalid type.");
		}
	}
}

void UBlendStackAnimNodeLibrary::BlendToWithSettings(const FAnimUpdateContext& Context,
													const FBlendStackAnimNodeReference& BlendStackNode, 
													UAnimationAsset* AnimationAsset /*= nullptr*/, 
													float AnimationTime /*= 0.f*/, 
													bool bLoop /*= false*/, 
													bool bMirrored /*= false*/, 
													float BlendTime /*= 0.2f*/, 
													UBlendProfile* BlendProfile /*= nullptr*/, 
													EAlphaBlendOption BlendOption /*= EAlphaBlendOption::HermiteCubic*/,
													bool bInertialBlend /*= false*/, 
													FVector BlendParameters /*= FVector::ZeroVector*/, 
													float WantedPlayRate /*= 1.f*/, 
													float ActivationDelay /*= 0.f*/)
{
	if (AnimationAsset != nullptr)
	{
		if (FAnimNode_BlendStack* BlendStackNodePtr = BlendStackNode.GetAnimNodePtr<FAnimNode_BlendStack>())
		{
			if (const FAnimationUpdateContext* AnimationUpdateContext = Context.GetContext())
			{
				BlendStackNodePtr->BlendTo(
					*AnimationUpdateContext,
					AnimationAsset,
					AnimationTime,
					bLoop,
					bMirrored,
					BlendStackNodePtr->MirrorDataTable,
					BlendTime,
					BlendProfile,
					BlendOption,
					bInertialBlend,
					BlendStackNodePtr->InertialBlendNodeTag,
					BlendParameters,
					WantedPlayRate,
					ActivationDelay,
					BlendStackNodePtr->GetGroupName(), BlendStackNodePtr->GetGroupRole(), BlendStackNodePtr->GetGroupMethod());
			}
			else
			{
				UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::BlendToWithSettings called with an invalid context.");
			}
		}
		else
		{
			UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::BlendToWithSettings called with an invalid type.");
		}
	}
}

UAnimationAsset* UBlendStackAnimNodeLibrary::GetCurrentAsset(const FBlendStackAnimNodeReference& BlendStackNode)
{
	if (FAnimNode_BlendStack* BlendStackNodePtr = BlendStackNode.GetAnimNodePtr<FAnimNode_BlendStack>())
	{
		return BlendStackNodePtr->GetAnimAsset();
	}

	UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::GetCurrentAsset called with an invalid type.");
	return nullptr;
}

float UBlendStackAnimNodeLibrary::GetCurrentAssetTime(const FBlendStackAnimNodeReference& BlendStackNode)
{
	if (FAnimNode_BlendStack* BlendStackNodePtr = BlendStackNode.GetAnimNodePtr<FAnimNode_BlendStack>())
	{
		return BlendStackNodePtr->GetCurrentAssetTime();
	}

	UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::GetCurrentAssetTime called with an invalid type.");
	return 0.0f;
}

float UBlendStackAnimNodeLibrary::GetCurrentAssetTimeRemaining(const FBlendStackAnimNodeReference& BlendStackNode)
{
	if (FAnimNode_BlendStack* BlendStackNodePtr = BlendStackNode.GetAnimNodePtr<FAnimNode_BlendStack>())
	{
		return BlendStackNodePtr->GetCurrentAssetLength() - BlendStackNodePtr->GetCurrentAssetTimePlayRateAdjusted();
	}

	UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::GetCurrentAssetTimeRemaining called with an invalid type.");
	return 0.0f;
}

bool UBlendStackAnimNodeLibrary::IsCurrentAssetLooping(const FBlendStackAnimNodeReference& BlendStackNode)
{
	if (FAnimNode_BlendStack* BlendStackNodePtr = BlendStackNode.GetAnimNodePtr<FAnimNode_BlendStack>())
	{
		return BlendStackNodePtr->IsLooping();
	}

	UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::IsCurrentAssetLooping called with an invalid type.");
	return false;
}

void UBlendStackAnimNodeLibrary::ForceBlendNextUpdate(const FBlendStackAnimNodeReference& BlendStackNode)
{
	if (FAnimNode_BlendStack* BlendStackNodePtr = BlendStackNode.GetAnimNodePtr<FAnimNode_BlendStack>())
	{
		BlendStackNodePtr->ForceBlendNextUpdate();
	}
	else
	{
		UE_LOGF(LogBlendStack, Warning, "UBlendStackAnimNodeLibrary::ForceBlendNextUpdate called with an invalid type.");
	}
}
