// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendStack/BlendStackInputAnimNodeLibrary.h"
#include "BlendStack/AnimNode_BlendStackInput.h"
#include "BlendStack/AnimNode_BlendStack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendStackInputAnimNodeLibrary)

FBlendStackInputAnimNodeReference UBlendStackInputAnimNodeLibrary::ConvertToBlendStackInputNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FBlendStackInputAnimNodeReference>(Node, Result);
}

void UBlendStackInputAnimNodeLibrary::GetProperties(const FBlendStackInputAnimNodeReference& BlendStackInputNode, UAnimationAsset*& AnimationAsset, float& AccumulatedTime)
{
	AnimationAsset = nullptr;
	AccumulatedTime = 0.f;

	if (const FAnimNode_BlendStackInput* BlendStackInputNodePtr = BlendStackInputNode.GetAnimNodePtr<FAnimNode_BlendStackInput>())
	{
		if (BlendStackInputNodePtr->Player)
		{
			if (const FBlendStackAnimPlayer* Player = *BlendStackInputNodePtr->Player)
			{
				AnimationAsset = Player->GetAnimationAsset();
				AccumulatedTime = Player->GetAccumulatedTime();
			}
		}
	}
}
