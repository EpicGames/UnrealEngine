// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendedCurveUtils.h"
#include "Animation/Skeleton.h"
#include "BoneContainer.h"
#include "HAL/IConsoleManager.h"

namespace UE::BlendStack
{
	
bool GVarUseBlendCurveFixes = true;
static FAutoConsoleVariableRef CVarUseBlendCurveFixes(TEXT("a.AnimNode.BlendStack.UseBlendCurveFixes"), GVarUseBlendCurveFixes, TEXT("Enable BlendStack BlendCurve fixes. It causes behavioral changes. For the good"));

// function adapted from TBaseBlendedCurve::LerpTo, that uses UnionEx instead of UE::Anim::FNamedValueArrayUtils::Union
void FBlendedCurveUtils::LerpToEx(FBlendedCurve& InOutCurve, const FBlendedCurve& OtherCurve, float Alpha)
{
	if (!FAnimWeight::IsRelevant(FMath::Abs(Alpha)))
	{
	}
	else if (!FAnimWeight::IsRelevant(FMath::Abs(Alpha - 1.0f)))
	{
		// if blend is all the way for child2, then just copy its bone atoms
		InOutCurve.Override(OtherCurve);
	}
	else
	{
		// Combine using Lerp. Result is a merged set of curves in 'this'. 
		UnionEx(InOutCurve, OtherCurve, [&Alpha](FBlendedCurve::ElementType& InOutThisElement, const FBlendedCurve::ElementType& InOtherElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				InOutThisElement.Value = FMath::Lerp(InOutThisElement.Value, InOtherElement.Value, Alpha);
				InOutThisElement.Flags |= InOtherElement.Flags;
			});
	}
}

// LerpToEx with per linked bone weighting blend
void FBlendedCurveUtils::LerpToPerBoneEx(FBlendedCurve& InOutCurve, const FBlendedCurve& OtherCurve, const FBoneContainer& BoneContainer, TConstArrayView<float> OtherCurveBoneWeights)
{
	// NoTe: BEHAVIOR CHANGE START
	//       as the note in the else branch states, the current behavior is not perfect, so for now we encapsulate this new behaviors behind CVar for A/B testing
	if (GVarUseBlendCurveFixes)
	{
		const USkeleton* Skeleton = BoneContainer.GetSkeletonAsset();

		UnionEx(InOutCurve, OtherCurve,
			[Skeleton, &BoneContainer, &OtherCurveBoneWeights](FBlendedCurve::ElementType& InOutThisElement, const FBlendedCurve::ElementType& InOtherElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				check(InOutThisElement.Name == InOtherElement.Name);

				float Weight = OtherCurveBoneWeights[0];
				if (const FCurveMetaData* CurveMetaData = Skeleton->GetCurveMetaData(InOutThisElement.Name))
				{
					for (const FBoneReference& LinkedBone : CurveMetaData->LinkedBones)
					{
						const FCompactPoseBoneIndex CompactPoseIndex = LinkedBone.GetCompactPoseIndex(BoneContainer);
						if (CompactPoseIndex != INDEX_NONE)
						{
							Weight = OtherCurveBoneWeights[CompactPoseIndex.GetInt()];
							break;
						}
					}
				}

				InOutThisElement.Value = FMath::Lerp(InOutThisElement.Value, InOtherElement.Value, Weight);
				InOutThisElement.Flags |= InOtherElement.Flags;
			});
	}
	// NoTe: BEHAVIOR CHANGE END
	else
	{
		// @note : This isn't perfect as curve can link to joint, and it would be the best to use that information
		// but that is very expensive option as we have to have another indirect look up table to search. 
		// For now, replacing with combine (non-zero will be overriden)
		// in the future, we might want to do this outside if we want per bone blend to apply curve also UE-39182
		InOutCurve.Combine(OtherCurve);
	}
}

} // namespace UE::BlendStack
