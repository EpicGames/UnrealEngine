// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimCurveTypes.h"

namespace UE::BlendStack
{

// Experimental, this feature might be removed without warning, not for production use
extern bool GVarUseBlendCurveFixes;

// Experimental, this feature might be removed without warning, not for production use
// Utils struct used to perform Curve blending with additiona fixes that have behavioral changes before integrating changes back to their original locations (look for code comments)
struct FBlendedCurveUtils : FBlendedCurve
{
	// function adapted from UE::Anim::FNamedValueArrayUtils::Union, with additional bugfixes (read comments) that changes behaviors
	template<typename PredicateType>
	static void UnionEx(FBlendedCurve& BlendedCurve0, const FBlendedCurve& BlendedCurve1, PredicateType InPredicate)
	{
		// reinterpret_cast hack to access FBlendedCurve protected properties and methods (SortElementsIfRequired, Elements)
		FBlendedCurveUtils& InOutValueArray0 = reinterpret_cast<FBlendedCurveUtils&>(BlendedCurve0);
		const FBlendedCurveUtils& InValueArray1 = reinterpret_cast<const FBlendedCurveUtils&>(BlendedCurve1);

		// Check arrays are not overlapping
		check((void*)&InOutValueArray0 != (void*)&InValueArray1);

		const int32 NumElements0 = InOutValueArray0.Num();	// ValueArray1 elements remain constant, but ValueArray0 can have entries added.
		const int32 NumElements1 = InValueArray1.Num();

		// A default element we re-use when an element from one of the two inputs is missing
		FBlendedCurve::ElementType DefaultElement;

		// Early out if we have no elements to union
		if (NumElements1 == 0)
		{
			if (GVarUseBlendCurveFixes)
			{
				// NoTe: BEHAVIOR CHANGE START
				//       FNamedValueArrayUtils::Union just returns, without applying InPredicate to the ElementPtr0(s),
				//       resulting into the curve not blending to the default value
				FBlendedCurve::ElementType* RESTRICT ElementPtr0 = InOutValueArray0.Elements.GetData();
				const FBlendedCurve::ElementType* RESTRICT ElementEndPtr0 = ElementPtr0 + NumElements0;

				while (ElementPtr0 != ElementEndPtr0)
				{
					DefaultElement.Name = ElementPtr0->Name;

					InPredicate(*ElementPtr0, DefaultElement, UE::Anim::ENamedValueUnionFlags::ValidArg0);

					++ElementPtr0;
				}
				// NoTe: BEHAVIOR CHANGE END
			}
		}
		else
		{
			// Sort both input arrays if required
			InOutValueArray0.SortElementsIfRequired();
			InValueArray1.SortElementsIfRequired();

			// Reserve memory for 1.5x combined curve counts.
			// This can overestimate in some circumstances, but it handles the common cases which are:
			// - One input is empty, the other not
			// - Both inputs are non-empty but do not share most elements
			int32 ReserveSize = FMath::Max(NumElements0, NumElements1);
			ReserveSize += ReserveSize / 2;
			InOutValueArray0.Reserve(ReserveSize);

			// Use pointers to iterate as this uses fewer registers and this code is very hot
			FBlendedCurve::ElementType* RESTRICT ElementPtr0 = InOutValueArray0.Elements.GetData();
			const FBlendedCurve::ElementType* RESTRICT ElementPtr1 = InValueArray1.Elements.GetData();

			const FBlendedCurve::ElementType* RESTRICT ElementEndPtr0 = ElementPtr0 + NumElements0;
			const FBlendedCurve::ElementType* RESTRICT ElementEndPtr1 = ElementPtr1 + NumElements1;

			// When we reach the end of either input arrays, we stop the tape merge and copy what remains
			bool bIsDone = ElementPtr0 == ElementEndPtr0 || ElementPtr1 == ElementEndPtr1;

			// Perform dual-iteration on the two sorted arrays
			while (!bIsDone)
			{
				if (ElementPtr0->Name == ElementPtr1->Name)
				{
					// Elements match, run predicate and increment both indices
					InPredicate(*ElementPtr0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::BothArgsValid);

					++ElementPtr0;
					++ElementPtr1;

					bIsDone = ElementPtr0 == ElementEndPtr0 || ElementPtr1 == ElementEndPtr1;
				}
				else if (ElementPtr0->Name.FastLess(ElementPtr1->Name))
				{
					// ValueArray0 element is earlier, so run predicate with only ValueArray0 and increment ValueArray0
					DefaultElement.Name = ElementPtr0->Name;

					InPredicate(*ElementPtr0, DefaultElement, UE::Anim::ENamedValueUnionFlags::ValidArg0);

					++ElementPtr0;

					bIsDone = ElementPtr0 == ElementEndPtr0;
				}
				else
				{
					// ValueArray1 element is earlier, so add to ValueArray0, run predicate with only second and increment ValueArray1
					const int32 ElementIndex0 = UE_PTRDIFF_TO_INT32(ElementPtr0 - InOutValueArray0.Elements.GetData());
					InOutValueArray0.Elements.InsertUninitialized(ElementIndex0);

					// Refresh pointers since they might have changed
					ElementPtr0 = InOutValueArray0.Elements.GetData() + ElementIndex0;
					ElementEndPtr0 = InOutValueArray0.Elements.GetData() + InOutValueArray0.Elements.Num();

					// We use placement new to make sure the constructor is inlined to reduce redundant work
					new(ElementPtr0) FBlendedCurve::ElementType();
					ElementPtr0->Name = ElementPtr1->Name;

					InPredicate(*ElementPtr0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::ValidArg1);

					++ElementPtr0;	// Increment this as well since we've inserted
					++ElementPtr1;

					bIsDone = ElementPtr1 == ElementEndPtr1;
				}
			}

			// Tape merge is done, copy anything that might be remaining
			if (ElementPtr1 < ElementEndPtr1)
			{
				// Reached end of ValueArray0 with remaining in ValueArray1, we can just copy the remainder of ValueArray1
				const int32 NumResults = InOutValueArray0.Elements.Num();
				const int32 NumNewElements = (ElementEndPtr1 - ElementPtr1);
				InOutValueArray0.Elements.Reserve(NumResults + NumNewElements);
				InOutValueArray0.Elements.AddUninitialized(NumNewElements);

				// Refresh pointers since they might have changed
				ElementPtr0 = InOutValueArray0.Elements.GetData() + NumResults;
				ElementEndPtr0 = ElementPtr0 + NumNewElements;

				for (; ElementPtr1 < ElementEndPtr1; ++ElementPtr0, ++ElementPtr1)
				{
					// We use placement new to make sure the constructor is inlined to reduce redundant work
					new(ElementPtr0) FBlendedCurve::ElementType();
					ElementPtr0->Name = ElementPtr1->Name;

					InPredicate(*ElementPtr0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::ValidArg1);
				}
			}

			// NoTe: BEHAVIOR CHANGE START
			if (GVarUseBlendCurveFixes && ElementPtr0 < ElementEndPtr0)
			{
				for (; ElementPtr0 < ElementEndPtr0; ++ElementPtr0)
				{
					DefaultElement.Name = ElementPtr0->Name;

					InPredicate(*ElementPtr0, DefaultElement, UE::Anim::ENamedValueUnionFlags::ValidArg0);
				}
			}
			// NoTe: BEHAVIOR CHANGE END
		}

		InOutValueArray0.CheckSorted();
	}

	// function adapted from TBaseBlendedCurve::LerpTo, that uses UnionEx instead of UE::Anim::FNamedValueArrayUtils::Union
	static void LerpToEx(FBlendedCurve& InOutCurve, const FBlendedCurve& OtherCurve, float Alpha);

	// LerpToEx with per linked bone weighting blend
	static void LerpToPerBoneEx(FBlendedCurve& InOutCurve, const FBlendedCurve& OtherCurve, const FBoneContainer& BoneContainer, TConstArrayView<float> OtherCurveBoneWeights);
};

} // namespace UE::BlendStack
