// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNode_AssetPlayerBase.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "AnimNodes/AnimNode_Mirror.h"
#include "EngineDefines.h"
#include "AnimNode_BlendStack.generated.h"

#define UE_API BLENDSTACK_API

UENUM()
enum class EBlendStack_BlendspaceUpdateMode : uint8
{
	// Only update the blendspace xy inputs once on blend in.
	InitialOnly,
	// Update the active/most recent blendspace xy inputs every frame.
	UpdateActiveOnly,
	// Update all blendspaces xy inputs every frame.
	UpdateAll,
};

USTRUCT()
struct FBlendStackAnimPlayer
{
	GENERATED_BODY()
	friend struct FAnimNode_BlendStack_Standalone;
	
	UE_API void Initialize(const FAnimationInitializeContext& Context, UAnimationAsset* AnimationAsset, float AccumulatedTime, bool bLoop,
		bool bMirrored, UMirrorDataTable* MirrorDataTable, float BlendTime, const UBlendProfile* InBlendProfile, EAlphaBlendOption InBlendOption,
		const FVector& BlendParameters, float PlayRate, float ActivationDelay, int32 InPoseLinkIdx, FName GroupName, EAnimGroupRole::Type GroupRole, EAnimSyncMethod Method, bool bOverridePositionWhenJoiningSyncGroupAsLeader);
	
	UE_API void UpdatePlayRate(float PlayRate);
	UE_API void Evaluate_AnyThread(FPoseContext& Output);
	UE_API void Update_AnyThread(const FAnimationUpdateContext& Context);
	UE_API float GetAccumulatedTime() const;
	UE_API float GetCurrentAssetTime() const;
	UE_API float GetCurrentAssetLength() const;
	UE_API float GetPlayRate() const;
	
	UE_API float GetBlendInPercentage() const;
	
	UE_API int32 GetBlendInWeightsNum() const;
	// Important: this method fills weight array indexed with the full skeleton indices, not the compact pose!
	UE_API void GetBlendInWeights(TArrayView<float> Weights) const;
	UE_API float GetBlendInWeight() const;

	EAlphaBlendOption GetBlendOption() const { return BlendOption; }
	UMirrorDataTable* GetMirrorDataTable() const;
	EAnimSyncMethod GetGroupMethod() const;
	FName GetGroupName() const;
	EAnimGroupRole::Type GetGroupRole() const;
	bool GetOverridePositionWhenJoiningSyncGroupAsLeader() const;
	const UBlendProfile* GetBlendProfile() const;

	UE_API void StorePoseContext(const FPoseContext& PoseContext);
	UE_API bool HasValidPoseContext() const;
	UE_API void MovePoseContextTo(FBlendStackAnimPlayer& Other);

	float GetTotalBlendInTime() const { return TotalBlendInTime; }
	float GetCurrentBlendInTime() const { return CurrentBlendInTime; }
	float GetTimeToActivation() const { return TimeToActivation; }
	
	UE_API void UpdateWithDeltaTime(float DeltaTime, int32 PlayerDepth, float PlayerDepthBlendInTimeMultiplier);
	bool GetMirror() const { return MirrorNode.GetMirror(); }
	UE_API FVector GetBlendParameters() const;
	UE_API void SetBlendParameters(const FVector& BlendParameters);
	UE_API FString GetAnimationName() const;
	UE_API UAnimationAsset* GetAnimationAsset() const;

	FAnimNode_Mirror_Standalone& GetMirrorNode() { return MirrorNode; }
	int32 GetPoseLinkIndex() const { return PoseLinkIndex; }

	UE_API void RestorePoseContext(FPoseContext& PoseContext) const;
	UE_API void UpdateSourceLinkNode();
	UE_API bool IsLooping() const;
	bool IsPostponed() const { return TimeToActivation > 0.f; };

	// A sample being "Active" is relative to the active branch of the animation update context
	// i.e. Active branches are NOT blending out.
	bool IsActive() const { return bIsActive; };

	UE_API FAnimNode_AssetPlayerBase* GetAssetPlayerNode();
	
	float GetSecondsFromEnd() const;

#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG 
	UE_API FColor GetDebugColor() const;
#endif

	// Curves to add to the pose after the player evaluates
	TBaseBlendedCurve<FDefaultAllocator, UE::Anim::FCurveElement> OverrideCurve;

private:
	// Embedded standalone player to play sequence
	UPROPERTY(Transient)
	FAnimNode_SequencePlayer_Standalone SequencePlayerNode;

	// Embedded standalone player to play blend spaces
	UPROPERTY(Transient)
	FAnimNode_BlendSpacePlayer_Standalone BlendSpacePlayerNode;

	// Embedded mirror node to handle mirroring
	UPROPERTY(Transient)
	FAnimNode_Mirror_Standalone MirrorNode;

	// if SequencePlayerNode.GetSequence() and BlendSpacePlayerNode.GetBlendSpace() are nullptr, 
	// instead of using SequencePlayerNode or BlendSpacePlayerNode (wrapped in MirrorNode),
	// the output FPoseContext will be from StoredBones, StoredCurve, StoredAttributes
	// NoTe: we don't need a full FCompactHeapPose, since we use StoredBoneContainer to cache the bone container
	TArray<FTransform> StoredBones;
	FBlendedHeapCurve StoredCurve;
	UE::Anim::FMeshAttributeContainer StoredAttributes;
	// We need to store the bone container, in case we have a LOD swap during a blend that uses the stored pose.
	FBoneContainer StoredBoneContainer;

	EAlphaBlendOption BlendOption = EAlphaBlendOption::Linear;
	int32 PoseLinkIndex = INDEX_NONE;
	TCustomBoneIndexArray<float, FSkeletonPoseBoneIndex> TotalBlendInTimePerBone;

	UPROPERTY(Transient)
	TObjectPtr<const UBlendProfile> BlendProfile = nullptr;

	float TotalBlendInTime = 0.f;
	float CurrentBlendInTime = 0.f;
	float TimeToActivation = 0.f;
	bool bIsActive = false;

#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG 
	FColor DebugColor = FColor::Black;
#endif
};

namespace UE::BlendStack
{
struct FBlendStack_SampleGraphExecutionHelper
{
	FBlendStackAnimPlayer* Player = nullptr;
	FGraphTraversalCounter CacheBoneCounter;

	void SetInputPosePlayer(FBlendStackAnimPlayer& InPlayer);
	void EvaluatePlayer(FPoseContext& Output, FBlendStackAnimPlayer& SamplePlayer, FPoseLink& SamplePoseLink);
	void ConditionalCacheBones(const FAnimationBaseContext& Output, FPoseLink& SamplePoseLink);
};
} // namespace UE::BlendStack

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendStack_Standalone : public FAnimNode_AssetPlayerBase
{
	GENERATED_BODY()

	TArray<UE::BlendStack::FBlendStack_SampleGraphExecutionHelper> SampleGraphExecutionHelpers;

	UPROPERTY()
	TArray<FPoseLink> PerSampleGraphPoseLinks;

	int32 CurrentSamplePoseLink = -1;

	UPROPERTY(Transient)
	TArray<FBlendStackAnimPlayer> AnimPlayers;

	// Flag that determines if any notifies from originating from an anim player samples should be filtered or not.
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bShouldFilterNotifies = false;
	
	// database used to search for an animation stitch to use as blend
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stitch|Experimental", meta = (PinHiddenByDefault))
	TObjectPtr<UObject> StitchDatabase;

	// blend time in seconds used to blend into and out from a stitch animation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stitch|Experimental", meta = (PinHiddenByDefault))
	float StitchBlendTime = 0.1f;

	// if the cost from searching StitchDatabase is above StitchBlendMaxCost, blend stack will perform a regular blend,
	// and not using the returned stitch animation as blend
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stitch|Experimental", meta = (PinHiddenByDefault))
	float StitchBlendMaxCost = 100.f;

	// FAnimNode_Base interface
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	UE_API void BlendTo(const FAnimationUpdateContext& Context, UAnimationAsset* AnimationAsset, float AccumulatedTime = 0.f, bool bLoop = false, 
		bool bMirrored = false, UMirrorDataTable* MirrorDataTable = nullptr, float BlendTime = 0.2f,
		const UBlendProfile* BlendProfile = nullptr, EAlphaBlendOption BlendOption = EAlphaBlendOption::Linear, 
		bool bUseInertialBlend = false, FName InertialBlendNodeTag = NAME_None, const FVector& BlendParameters = FVector::Zero(), float PlayRate = 1.f, float ActivationDelay = 0.f,
		FName GroupName = NAME_None, EAnimGroupRole::Type GroupRole = EAnimGroupRole::CanBeLeader, EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync, bool bOverridePositionWhenJoiningSyncGroupAsLeader = false,
		FName PrevGroupName = NAME_None, EAnimGroupRole::Type PrevGroupRole = EAnimGroupRole::CanBeLeader, EAnimSyncMethod PrevMethod = EAnimSyncMethod::DoNotSync);
	UE_API void UpdatePlayRate(float PlayRate);
	UE_API virtual void Reset();

	// FAnimNode_AssetPlayerBase interface
	UE_API virtual UAnimationAsset* GetAnimAsset() const;
	UE_API virtual float GetAccumulatedTime() const override;
	UE_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	UE_API virtual float GetCurrentAssetTime() const override;
	UE_API virtual float GetCurrentAssetLength() const override;
	// End of FAnimNode_AssetPlayerBase interface

	UE_API bool GetMirror() const;
	UE_API FVector GetBlendParameters() const;

	UE_API int32 GetNextPoseLinkIndex();

	int32 GetMaxActiveBlends() const { return MaxActiveBlends; }
	void SetMaxActiveBlends(int32 InMaxActiveBlends) { MaxActiveBlends = InMaxActiveBlends; }
	
	// Experimental, this feature might be removed without warning, not for production use
	UE_API bool AnyNewBlendToThisFrame() const;

protected:
	UE_API void InternalBlendTo(const FAnimationUpdateContext& Context, UAnimationAsset* AnimationAsset, float AccumulatedTime, bool bLoop, 
		bool bMirrored, UMirrorDataTable* MirrorDataTable, float BlendTime,
		const UBlendProfile* BlendProfile, EAlphaBlendOption BlendOption, 
		bool bUseInertialBlend, FName InertialBlendNodeTag, const FVector& BlendParameters, float PlayRate, float ActivationDelay,
		FName GroupName, EAnimGroupRole::Type GroupRole, EAnimSyncMethod GroupMethod, bool bOverridePositionWhenJoiningSyncGroupAsLeader, 
		FName PrevGroupName, EAnimGroupRole::Type PrevGroupRole, EAnimSyncMethod PrevGroupMethod);

	// Call this to update current blendspace player xy blend parameters. By default, we only update them on initial BlendTo.
	UE_API void UpdateBlendspaceParameters(const EBlendStack_BlendspaceUpdateMode UpdateMode, const FVector& BlendParameters);

	static UE_API void BlendWithPose(FAnimationPoseData& InOutPoseData, const FAnimationPoseData& OtherPoseData, const float InOutPoseWeight);
	static UE_API void BlendWithPosePerBone(FAnimationPoseData& InOutPoseData, const FAnimationPoseData& OtherPoseData, TConstArrayView<float> OtherPoseWeights);

	// Number of max active blending animation in the blend stack. If MaxActiveBlends is zero then blend stack is disabled
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0"))
	int32 MaxActiveBlends = 4;

	// if the number of requested blends is higher than MaxActiveBlends, blend stack will blend and accumulate 
	// into a stored pose all the overflowing animations. if bStoreBlendedPose is false, the memory to store the pose will be saved,
	// but once reached the MaxActiveBlends, blendstack will start discarding animations, potentially resulting in animation pops
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bStoreBlendedPose = true;

    TSharedPtr<TArray<FName>> NotifiesFiredLastTick;
	TSharedPtr<TMap<FName,float>> NotifyRecencyMap;

	// Window of time after firing a notify that any instance of the same notify will be filtered out.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0", PinHiddenByDefault))
	float NotifyRecencyTimeOut = 0.2f;

	// if the most relevant (recently added) animation is within MaxBlendInTimeToOverrideAnimation, the new requested blend will take its spot
	// otherwise a new blended will be added to the stack
	UPROPERTY(EditAnywhere, Category = Settings)
	float MaxBlendInTimeToOverrideAnimation = 0.f;

	// AnimPlayers blend in timer will be incremented PlayerDepthBlendInTimeMultiplier times faster on a deeper blend
	UPROPERTY(EditAnywhere, Category = Settings)
	float PlayerDepthBlendInTimeMultiplier = 1.f;

private:
	bool StitchTo(const FAnimationUpdateContext& Context, UAnimationAsset* AnimationAsset, float AccumulatedTime, bool bLoop,
		bool bMirrored, UMirrorDataTable* MirrorDataTable, float BlendTime, const UBlendProfile* BlendProfile, EAlphaBlendOption BlendOption, bool bUseInertialBlend, FName InertialBlendNodeTag,
		const FVector& BlendParameters, float PlayRate, float ActivationDelay);

	void PopLastAnimPlayer();
	void InitializeSample(const FAnimationInitializeContext& Context, FBlendStackAnimPlayer& SamplePlayer);
	bool EvaluateSample(FPoseContext& Output, const int32 PlayerIndex);
	void UpdateSample(const FAnimationUpdateContext& Context, const int32 PlayerIndex);
	void CacheBonesForSample(const FAnimationCacheBonesContext& Context, const int32 PlayerIndex);
	bool IsSampleGraphAvailableForPlayer(const int32 PlayerIndex);
	
	uint64 LastBlendToFrameCounter = 0;
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendStack : public FAnimNode_BlendStack_Standalone
{
	GENERATED_BODY()

	// requested animation to play
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	TObjectPtr<UAnimationAsset> AnimationAsset;

	// requested animation time
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float AnimationTime = -1.f;

	// delay in seconds before activating AnimationAsset playing from AnimationTime
	// assets queued with an ActivationDelayTime will be discarded when a new blend gets requested
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float ActivationDelayTime = 0.f;

	// requested AnimationAsset looping
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	bool bLoop = true;

	// requested AnimationAsset mirroring
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	bool bMirrored = false;

	// requested animation play rate
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float WantedPlayRate = 1.f;
	
	// tunable animation transition blend time 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault, ClampMin = "0"))
	float BlendTime = 0.2f;

	// if MaxAnimationDeltaTime is positive and the currently playing animation accumulated time differs more than MaxAnimationDeltaTime from AnimationTime
	// (animation desynchronized from the requested time) this blend stack will force a blend into the same animation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float MaxAnimationDeltaTime = -1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault, UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> BlendProfile;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	EAlphaBlendOption BlendOption = UE::Anim::DefaultBlendOption;

	// How we should update individual blend space parameters. See dropdown options tooltips.
	UPROPERTY(EditAnywhere, Category = Blendspace)
	EBlendStack_BlendspaceUpdateMode BlendspaceUpdateMode = EBlendStack_BlendspaceUpdateMode::InitialOnly;

	// requested blend space blend parameters (if AnimationAsset is a blend space)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Blendspace, meta = (PinHiddenByDefault))
	FVector BlendParameters = FVector::Zero();

	// if set and bMirrored MirrorDataTable will be used for mirroring the aniamtion
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	TObjectPtr<UMirrorDataTable> MirrorDataTable;

	// Use this to define a threshold to trigger a new blend when blendspace xy input pins change.
	// By default, any delta will trigger a blend.
	UPROPERTY(EditAnywhere, Category = Blendspace)
	float BlendParametersDeltaThreshold = 0.0f;

	// in case there're no blend to the current frame, this blend stack will force a blend to into the currently playing animation. 
	// useful to blend into the same animation in case this blendstack synch properties are set to be a follower
	UPROPERTY(Experimental, EditAnywhere, BlueprintReadWrite, Category = "Experimental", meta = (PinHiddenByDefault))
	bool bForceBlendTo = false;

	// Enable to use inertial blending
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bUseInertialBlend = false;

	// Tag to force a specific inertialization / dead blending node to process inertial blend requests coming from this blend stack
	UPROPERTY(EditAnywhere, Category = Settings)
	FName InertialBlendNodeTag = NAME_None;

	// Reset the blend stack if it has become relevant to the graph after not being updated on previous frames.
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bResetOnBecomingRelevant = true;

#if WITH_EDITORONLY_DATA
    // The group name that we synchronize with (NAME_None if it is not part of any group). Note that
    // this is the name of the group used to sync the output of this node - it will not force
    // syncing of animations contained by it.
    UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
    FName GroupName = NAME_None;
    
    // The role this node (and main AnimPlayer - most recent animation set via BlendTo) can assume within the group (ignored if GroupName is not set)
    UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
    TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;
    
    // How this node main AnimPlayer will synchronize with other animations
    UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
    EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;
    
    // The role set to the most relevant non main AnimPlayer (previous main AnimPlayer)
    UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
    FName PrevGroupName = NAME_None;
    
	// The group set to the most relevant non main AnimPlayer (previous main AnimPlayer)
    UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
    TEnumAsByte<EAnimGroupRole::Type> PrevGroupRole = EAnimGroupRole::CanBeLeader;
    
	// The method set to the most relevant non main AnimPlayer (previous main AnimPlayer)
    UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
    EAnimSyncMethod PrevMethod = EAnimSyncMethod::DoNotSync;

    // If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
    UPROPERTY(EditAnywhere, Category = Relevancy, meta = (FoldProperty, PinHiddenByDefault))
    bool bIgnoreForRelevancyTest = false;
#endif // WITH_EDITORONLY_DATA

	// FAnimNode_Base interface
	UE_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	UE_API virtual FName GetGroupName() const override;
	UE_API virtual EAnimGroupRole::Type GetGroupRole() const override;
	UE_API virtual EAnimSyncMethod GetGroupMethod() const override;

	UE_API FName GetPrevGroupName() const;
	UE_API EAnimGroupRole::Type GetPrevGroupRole() const;
	UE_API EAnimSyncMethod GetPrevGroupMethod() const;

	UE_API virtual bool GetIgnoreForRelevancyTest() const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool SetGroupName(FName InGroupName) override;
	UE_API virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override;
	UE_API virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override;
	UE_API virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override;
	// End of FAnimNode_Base interface

	// Force a blend on the next update, even if the anim sequence has not changed.
	UE_API void ForceBlendNextUpdate();
	UE_API virtual void Reset() override;

protected:
	UE_API bool NeedsReset(const FAnimationUpdateContext& Context) const;
	UE_API bool ConditionalBlendTo(const FAnimationUpdateContext& Context);

	// Update Counter for detecting being relevant
	FGraphTraversalCounter UpdateCounter;

	bool bForceBlendNextUpdate = false;
};

#undef UE_API
