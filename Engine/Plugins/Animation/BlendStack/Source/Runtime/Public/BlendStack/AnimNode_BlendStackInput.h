// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"
#include "AnimNode_BlendStackInput.generated.h"

#define UE_API BLENDSTACK_API

/** Input pose that links the blend stack's sample graph with the sample/pose chosen by the blend stack.
*	@Todo: It might be better to reuse FAnimNode_LinkedInputPose, since we will most likely need variable input pins in the future too.
*/
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendStackInput : public FAnimNode_Base
{
	GENERATED_BODY()
	
public:

	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	
	UPROPERTY()
	int32 SampleIndex = INDEX_NONE;

	UPROPERTY()
	int32 BlendStackAllocationIndex = INDEX_NONE;

	// If true, the PlayRate input from thos node will override the SequencePlayer or BlendSpacePlayer playrate each frame
	UPROPERTY(EditAnywhere, Category=Settings, meta = (NeverAsPin))
	bool bOverridePlayRate = false;
	
	// The play rate multiplier. Can be negative, which will cause the animation to play in reverse.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	float PlayRate = 0.0f;	

	// The player is guaranteed to be valid for the whole duration of update/eval.
	struct FBlendStackAnimPlayer** Player;
};

#undef UE_API
