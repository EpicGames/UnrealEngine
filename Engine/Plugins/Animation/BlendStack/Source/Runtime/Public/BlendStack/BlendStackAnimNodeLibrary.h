// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNodeReference.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AlphaBlend.h"
#include "BlendStackAnimNodeLibrary.generated.h"

#define UE_API BLENDSTACK_API

struct FAnimNode_BlendStack;
class UBlendProfile;

USTRUCT(BlueprintType)
struct FBlendStackAnimNodeReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_BlendStack FInternalNodeType;
};

// Exposes operations that can be run on a Blend Stack node via Anim Node Functions such as "On Become Relevant" and "On Update".
UCLASS(MinimalAPI, Experimental)
class UBlendStackAnimNodeLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a blend stack node context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static UE_API FBlendStackAnimNodeReference ConvertToBlendStackNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);
	
	/** Get the current AnimationAsset that is playing from a Blend Stack Input node */
    UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
    static UE_API UAnimationAsset* GetCurrentBlendStackAnimAsset(const FAnimNodeReference& Node);
	
	/** Get the current elapsed time of the animation that is playing from a Blend Stack Input node */
    UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
    static UE_API float GetCurrentBlendStackAnimAssetTime(const FAnimNodeReference& Node);

	/** Get if current anim is active */
	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
	static UE_API bool GetCurrentBlendStackAnimIsActive(const FAnimNodeReference& Node);

	/** Get if we are currently mirrored from a Blend Stack Input node */
	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
	static UE_API bool GetCurrentBlendStackAnimAssetMirrored(const FAnimNodeReference& Node);

	/** Get current AssetMirrorTable from a Blend Stack Input node */
	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
	static UE_API UMirrorDataTable* GetCurrentBlendStackAnimAssetMirrorTable(const FAnimNodeReference& Node);

	/** Get a blend stack node context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe, DisplayName = "Convert to Blend Stack Node"))
	static void ConvertToBlendStackNodePure(const FAnimNodeReference& Node, FBlendStackAnimNodeReference& BlendStackNode, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		BlendStackNode = ConvertToBlendStackNode(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	UFUNCTION(BlueprintCallable, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe, DisplayName = "Force Blend On Next Update"))
	static UE_API void ForceBlendNextUpdate(const FBlendStackAnimNodeReference& BlendStackNode);

	UFUNCTION(BlueprintCallable, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe, DisplayName = "Blend To"))
	static UE_API void BlendTo(const FAnimUpdateContext& Context, 
						const FBlendStackAnimNodeReference& BlendStackNode, 
						UAnimationAsset* AnimationAsset = nullptr,
						float AnimationTime = 0.f,
						bool bLoop = false,
						bool bMirrored = false,
						float BlendTime = 0.2f,
						FVector BlendParameters = FVector::ZeroVector,
						float WantedPlayRate = 1.f,
						float ActivationDelay = 0.f);

	// Note: Experimental and subject to change!
	UFUNCTION(BlueprintCallable, Category = "Animation|BlendStack|Experimental", meta = (BlueprintThreadSafe, DisplayName = "Blend To"))
	static UE_API void BlendToWithSettings(const FAnimUpdateContext& Context, 
						const FBlendStackAnimNodeReference& BlendStackNode, 
						UAnimationAsset* AnimationAsset = nullptr,
						float AnimationTime = 0.f,
						bool bLoop = false,
						bool bMirrored = false,
						float BlendTime = 0.2f,
						UBlendProfile* BlendProfile = nullptr,
						EAlphaBlendOption BlendOption = EAlphaBlendOption::HermiteCubic,
						bool bInertialBlend = false,
						FVector BlendParameters = FVector::ZeroVector,
						float WantedPlayRate = 1.f,
						float ActivationDelay = 0.f);

	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
    static UE_API UAnimationAsset* GetCurrentAsset(const FBlendStackAnimNodeReference& BlendStackNode);

	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
    static UE_API float GetCurrentAssetTime(const FBlendStackAnimNodeReference& BlendStackNode);

	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
    static UE_API float GetCurrentAssetTimeRemaining(const FBlendStackAnimNodeReference& BlendStackNode);

	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
    static UE_API bool IsCurrentAssetLooping(const FBlendStackAnimNodeReference& BlendStackNode);
};

#undef UE_API
