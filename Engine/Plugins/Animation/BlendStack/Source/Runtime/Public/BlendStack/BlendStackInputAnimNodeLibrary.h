// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNodeReference.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "BlendStackInputAnimNodeLibrary.generated.h"

#define UE_API BLENDSTACK_API

struct FAnimNode_BlendStackInput;

USTRUCT(BlueprintType)
struct FBlendStackInputAnimNodeReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_BlendStackInput FInternalNodeType;
};

// Exposes operations that can be run on a Blend Stack node via Anim Node Functions such as "On Become Relevant" and "On Update".
UCLASS(MinimalAPI, Experimental)
class UBlendStackInputAnimNodeLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a blend stack input node context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static UE_API FBlendStackInputAnimNodeReference ConvertToBlendStackInputNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a blend stack input node context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe, DisplayName = "Convert to Blend Stack Input Node"))
	static void ConvertToBlendStackInputNodePure(const FAnimNodeReference& Node, FBlendStackInputAnimNodeReference& BlendStackInputNode, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		BlendStackInputNode = ConvertToBlendStackInputNode(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	UFUNCTION(BlueprintCallable, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
	static UE_API void GetProperties(const FBlendStackInputAnimNodeReference& BlendStackInputNode, UAnimationAsset*& AnimationAsset, float& AccumulatedTime);
};

#undef UE_API
